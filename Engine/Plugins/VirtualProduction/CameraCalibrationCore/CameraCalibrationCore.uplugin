{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Camera Calibration Core",
	"Description": "Supports lens distortion and camera calibration.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Hidden": true,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "CameraCalibrationCore",
			"Type": "Runtime",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "CameraCalibrationCoreEditor",
			"Type": "Editor",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GameplayCameras",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		}
	]
}
