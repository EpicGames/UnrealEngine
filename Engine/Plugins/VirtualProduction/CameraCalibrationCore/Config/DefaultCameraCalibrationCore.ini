[CoreRedirects]

;
; 4.27 Redirects
;

; Classes

+ClassRedirects=(OldName="/Script/LensDistortion.LensFile", NewName="/Script/CameraCalibrationCore.LensFile")
+ClassRedirects=(OldName="/Script/LensDistortion.LensDistortionModelHandlerBase", NewName="/Script/CameraCalibrationCore.LensDistortionModelHandlerBase")
+ClassRedirects=(OldName="/Script/LensDistortion.SphericalLensDistortionModelHandler", NewName="/Script/CameraCalibrationCore.SphericalLensDistortionModelHandler")
+ClassRedirects=(OldName="/Script/LensDistortion.LensModel", NewName="/Script/CameraCalibrationCore.LensModel")
+ClassRedirects=(OldName="/Script/LensDistortion.SphericalLensModel", NewName="/Script/CameraCalibrationCore.SphericalLensModel")

+ClassRedirects=(OldName="/Script/CameraCalibration.LensFile", NewName="/Script/CameraCalibrationCore.LensFile")
+ClassRedirects=(OldName="/Script/CameraCalibration.LensDistortionModelHandlerBase", NewName="/Script/CameraCalibrationCore.LensDistortionModelHandlerBase")
+ClassRedirects=(OldName="/Script/CameraCalibration.SphericalLensDistortionModelHandler", NewName="/Script/CameraCalibrationCore.SphericalLensDistortionModelHandler")
+ClassRedirects=(OldName="/Script/CameraCalibration.LensModel", NewName="/Script/CameraCalibrationCore.LensModel")
+ClassRedirects=(OldName="/Script/CameraCalibration.SphericalLensModel", NewName="/Script/CameraCalibrationCore.SphericalLensModel")

; Packages

+PackageRedirects=(OldName="/LensDistortion/Materials/M_SphericalDistortionPostProcess", NewName="/CameraCalibrationCore/Materials/M_DistortionPostProcess")
+PackageRedirects=(OldName="/LensDistortion/Materials/M_SphericalDistortionDisplacementMap", NewName="/CameraCalibrationCore/Materials/M_SphericalDistortionDisplacementMap")

+PackageRedirects=(OldName="/CameraCalibration/Materials/M_SphericalDistortionPostProcess", NewName="/CameraCalibrationCore/Materials/M_DistortionPostProcess")
+PackageRedirects=(OldName="/CameraCalibration/Materials/M_SphericalDistortionDisplacementMap", NewName="/CameraCalibrationCore/Materials/M_SphericalDistortionDisplacementMap")

;
; 5.1 Redirects
;

; Classes

+ClassRedirects=(OldName="LensDistortionComponent", NewName="/Script/CameraCalibrationCore.LensComponent")

; Packages

+PackageRedirects=(OldName="/Script/CameraCalibration", Removed=true)

; Properties

+PropertyRedirects=(OldName="LensComponent.bApplyNodalOffset",NewName="LensComponent.bApplyNodalOffsetOnTick")


;
; 5.7 redirects
;
+ClassRedirects=(OldName="/Script/GameplayCameras.LensCalibrationCameraNode", NewName="/Script/CameraCalibrationCore.LensCalibrationCameraNode")

;
; 5.8 redirects
;
+EnumRedirects=(OldName="EDistortionRenderingMode", ValueChanges=(("SceneViewExtension", "TemporalSuperResolution")))