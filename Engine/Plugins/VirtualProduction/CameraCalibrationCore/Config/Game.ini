[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysCook=(Path="/CameraCalibrationCore")