// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Implements the 3DE4 Anamorphic - Standard, Degree 4 model for distortion.
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

// Shared parameters
float2 ThreadIdToUV;
float InverseOverscan;
float CameraOverscan;
RWTexture2D<float2> OutDistortionMap;

// Anamorphic distortion parameters (3DE4 Anamorphic - Standard, Degree 4 model)
float PixelAspect;
float2 FilmbackSize;
float2 Squeeze;
float LensRotation;
float CX02;
float CX04;
float CX22;
float CX24;
float CX44;
float CY02;
float CY04;
float CY22;
float CY24;
float CY44;

float2 AnamorphicDistort(float2 InputUV)
{
	// The following implementation is based on the 3DE4 Anamorphic - Standard, Degree 4 model for distortion
	// Pre-compute some values that will be useful later
	float r_fb = (0.5) * sqrt(FilmbackSize.x*FilmbackSize.x + FilmbackSize.y*FilmbackSize.y);

	float phi_mnt_rad = LensRotation * (PI / 180);
	float cos_phi_mnt = cos(phi_mnt_rad);
	float sin_phi_mnt = sin(phi_mnt_rad);

	// Step 1: Transform input UVs into diagonally-normalized coordinates, based on the width and height of the filmback/image
	float x_dn = (InputUV.x - 0.5) * (FilmbackSize.x / r_fb); // don't forget to factor in lens center offset later
	float y_dn = (InputUV.y - 0.5) * (FilmbackSize.y / r_fb); // don't forget to factor in lens center offset later

	// Step 2: Scale the x-coordinate by the inverse of the pixel aspect ratio
	x_dn = x_dn / PixelAspect;

	// Step 3: Rotate x and y by the inverse of the mounting angle
	float x_dn_rotated = (x_dn * cos_phi_mnt) + (y_dn * sin_phi_mnt);
	float y_dn_rotated = (y_dn * cos_phi_mnt) - (x_dn * sin_phi_mnt);

	// Step 4: Compute the anamorphic distortion for x and y, to the 4th degree
	float r_dn_sq = x_dn_rotated*x_dn_rotated + y_dn_rotated*y_dn_rotated;
	float r_dn_4th = r_dn_sq * r_dn_sq;
	float phi = atan2(y_dn_rotated, x_dn_rotated);

	float x_dn_prime = x_dn_rotated * (1 + (CX02 * r_dn_sq) + (CX04 * r_dn_4th) + (CX22 * r_dn_sq * cos(2 * phi)) + (CX24 * r_dn_4th * cos(2 * phi)) + (CX44 * r_dn_4th * cos(4 * phi)));
	float y_dn_prime = y_dn_rotated * (1 + (CY02 * r_dn_sq) + (CY04 * r_dn_4th) + (CY22 * r_dn_sq * cos(2 * phi)) + (CY24 * r_dn_4th * cos(2 * phi)) + (CY44 * r_dn_4th * cos(4 * phi)));

	// Step 5: Scale the x-coordinate by the pixel aspect ratio, and the x- and y-coordinates by the lens-breathing squeeze factors
	x_dn_prime = x_dn_prime * PixelAspect * Squeeze.x;
	y_dn_prime = y_dn_prime * Squeeze.y;

	// Step 6: Rotate x and y by the mounting angle
	float x_dn_prime_rotated = (x_dn_prime * cos_phi_mnt) - (y_dn_prime * sin_phi_mnt);
	float y_dn_prime_rotated = (y_dn_prime * cos_phi_mnt) + (x_dn_prime * sin_phi_mnt);

	// Step 7: Transform diagonally normalized coordinates back into unit coordinates
	float x_unit_prime = ((x_dn_prime_rotated * r_fb) / FilmbackSize.x) + 0.5;
	float y_unit_prime = ((y_dn_prime_rotated * r_fb) / FilmbackSize.y) + 0.5;
	
	float2 output_uv;
	output_uv.x = x_unit_prime;
	output_uv.y = y_unit_prime;

	return output_uv;
}

float2 Distort(float2 InputUV)
{
	return AnamorphicDistort(InputUV);
}

bool IsDistortionWellOriented(float2 InputUV, float2 DistortedUV, float2 UVOffset)
{
	float2 DistortedXOffset = Distort(InputUV + UVOffset * int2(1, 0));
	float2 DistortedYOffset = Distort(InputUV + UVOffset * int2(0, 1));

	float2 XOffset = DistortedXOffset - DistortedUV;
	float2 YOffset = DistortedYOffset - DistortedUV;
	
	bool bIsFrontFacing = (XOffset.x * YOffset.y - XOffset.y * YOffset.x) >= 0;
	bool bIsRightSideUp = (XOffset.x * UVOffset.x) >= 0 && (YOffset.y * UVOffset.y) >= 0;
	return bIsFrontFacing && bIsRightSideUp;
}

float2 TryDistort(float2 InputUV)
{
	float2 DistortedPosition = Distort(InputUV);
	float2 UVOffset = 0.5 * ThreadIdToUV;
	bool bIsOrientedWell =
		IsDistortionWellOriented(InputUV, DistortedPosition, UVOffset) &&
		IsDistortionWellOriented(InputUV, DistortedPosition, -UVOffset);

	return lerp(InputUV, DistortedPosition, bIsOrientedWell);
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Add half-pixel offset to ThreadID to get accurate UV position
	float2 UndistortedUV = (DispatchThreadId.xy + 0.5) * ThreadIdToUV;
	
	// Rescale the UV by an overscan factor (>= 1.0) to widen the range of UVs to be distorted
	UndistortedUV = ((UndistortedUV - 0.5) * InverseOverscan * CameraOverscan) + 0.5;
	
	float2 DistortedUV = TryDistort(UndistortedUV);
	
	// Compute the UV displacement and scale it by the CameraOverscan to account for the increased resolution of the overscanned scene texture 
	OutDistortionMap[DispatchThreadId.xy] = (DistortedUV - UndistortedUV) / CameraOverscan;
}