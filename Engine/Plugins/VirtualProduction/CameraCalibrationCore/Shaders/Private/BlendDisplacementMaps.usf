// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlendDisplacementMaps.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

float2 ThreadIdToUV;
float2 FxFyScale;

// Stores the x and y values and the x and y tangents for each corner of the blending patch
// Ordered (X0, Y0) -> (X1, Y0) -> (X1, Y1) -> (X0, Y1)
float4 PatchCorners[4];

float EvalFocus;
float EvalZoom;

Texture2D<float2> InputDistortionMap1;
Texture2D<float2> InputDistortionMap2;
Texture2D<float2> InputDistortionMap3;
Texture2D<float2> InputDistortionMap4;

SamplerState SourceTextureSampler;

RWTexture2D<float2> OverscanDistortionMap;

float2 BezierInterp(float2 P0, float2 P1, float2 P2, float2 P3, float Alpha)
{
	float2 P01 = lerp(P0, P1, Alpha);
	float2 P12 = lerp(P1, P2, Alpha);
	float2 P23 = lerp(P2, P3, Alpha);
	float2 P012 = lerp(P01, P12, Alpha);
	float2 P123 = lerp(P12, P23, Alpha);
	float2 P0123 = lerp(P012, P123, Alpha);

	return P0123;
}

float2 EvalAtTwoPoints(float EvalTime, float Time0, float Time1, float2 Value0, float2 Value1, float Tangent0, float Tangent1)
{
	if (Time0 == Time1)
	{
		return Value0;
	}

	float OneThird = 1.0f / 3.0f;

	float Diff = Time1 - Time0;
	float Alpha = (EvalTime - Time0) / Diff;

	float2 DeltaValue = Value1 - Value0;
	float2 TangentScale = DeltaValue / Diff;
	float2 ScaledTangent0 = Tangent0 * TangentScale;
	float2 ScaledTangent1 = Tangent1 * TangentScale;

	float2 P0 = Value0;
	float2 P3 = Value1;
	float2 P1 = P0 + (ScaledTangent0 * Diff * OneThird);
	float2 P2 = P3 - (ScaledTangent1 * Diff * OneThird);

	return BezierInterp(P0, P1, P2, P3, Alpha);
}

float2 CoonsPatchBlend(float X, float Y, float2 Values[4])
{
	float Alpha = 0.0;
	if (PatchCorners[0].x != PatchCorners[1].x)
	{
		Alpha = (X - PatchCorners[0].x) / (PatchCorners[1].x - PatchCorners[0].x);
	}
	
	float Beta = 0.0;
	if (PatchCorners[0].y != PatchCorners[3].y)
	{
		Beta = (Y - PatchCorners[0].y) / (PatchCorners[3].y - PatchCorners[0].y);
	}
	
	float2 X0Curve = EvalAtTwoPoints(X, PatchCorners[0].x, PatchCorners[1].x, Values[0], Values[1], PatchCorners[0].z, PatchCorners[1].z);
	float2 X1Curve = EvalAtTwoPoints(X, PatchCorners[3].x, PatchCorners[2].x, Values[3], Values[2], PatchCorners[3].z, PatchCorners[2].z);
	float2 Y0Curve = EvalAtTwoPoints(Y, PatchCorners[0].y, PatchCorners[3].y, Values[0], Values[3], PatchCorners[0].w, PatchCorners[3].w);
	float2 Y1Curve = EvalAtTwoPoints(Y, PatchCorners[1].y, PatchCorners[2].y, Values[1], Values[2], PatchCorners[1].w, PatchCorners[2].w);
	
	float2 Lx = lerp(X0Curve, X1Curve, Beta);
	float2 Ly = lerp(Y0Curve, Y1Curve, Alpha);
	float2 B = lerp(lerp(Values[0], Values[1], Alpha), lerp(Values[3], Values[2], Alpha), Beta);

	return Lx + Ly - B;
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	float2 DestViewportUV = (DispatchThreadId.xy + 0.5) * ThreadIdToUV;

	// FxFyScale alters the UV to sample the input maps to account for filmback cropping
	// When the filmback is cropped, we only need to use a center crop of the distortion displacement map
	DestViewportUV = ((DestViewportUV - 0.5) * FxFyScale) + 0.5;

	float2 DistortionBlend;
	float2 Distortion1 = InputDistortionMap1.SampleLevel(SourceTextureSampler, DestViewportUV, 0);

#if BLEND_TYPE == 1 // Case - TwoFocusOneZoom - Requires two input distortion maps representing two focus points that share the same zoom value
	float2 Distortion2 = InputDistortionMap2.SampleLevel(SourceTextureSampler, DestViewportUV, 0);
	DistortionBlend = EvalAtTwoPoints(EvalFocus, PatchCorners[0].y, PatchCorners[1].y, Distortion1, Distortion2, PatchCorners[0].w, PatchCorners[1].w);
	
#elif BLEND_TYPE == 2 // Case - OneFocusTwoZoom - Requires two input distortion maps representing two zoom points that share the same focus value
	float2 Distortion2 = InputDistortionMap2.SampleLevel(SourceTextureSampler, DestViewportUV, 0);
	DistortionBlend = EvalAtTwoPoints(EvalZoom, PatchCorners[0].x, PatchCorners[1].x, Distortion1, Distortion2, PatchCorners[0].z, PatchCorners[1].z);
	
#elif BLEND_TYPE == 3 // Case - TwoFocusTwoZoom - Requires four input distortion maps representing four distinct focus/zoom pairs
	float2 Distortions[4] =
	{
		Distortion1,
		InputDistortionMap2.SampleLevel(SourceTextureSampler, DestViewportUV, 0),
		InputDistortionMap3.SampleLevel(SourceTextureSampler, DestViewportUV, 0),
		InputDistortionMap4.SampleLevel(SourceTextureSampler, DestViewportUV, 0)
	};

	DistortionBlend = CoonsPatchBlend(EvalZoom, EvalFocus, Distortions);

#else // Case - OneFocusOneZoom - Requires one input distortion map which is passed through directly (after being scaled for filmback cropping)
	DistortionBlend = Distortion1;
#endif
	
	OverscanDistortionMap[DispatchThreadId.xy] = DistortionBlend;		
}
