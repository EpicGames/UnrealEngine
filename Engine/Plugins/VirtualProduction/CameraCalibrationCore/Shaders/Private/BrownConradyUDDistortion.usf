// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Implements the Brown-Conrady U-D model with 6 radial and 2 tangential components.
	Uses polynomial division model: (1 + K1*r^2 + K2*r^4 + K3*r^6) / (1 + K4*r^2 + K5*r^4 + K6*r^6)
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

// Shared parameters
float2 ThreadIdToUV;
float InverseOverscan;
float CameraOverscan;
RWTexture2D<float2> OutDistortionMap;
int bIsUD;

// Brown-Conrady U-D distortion parameters (polynomial division model)
float2 FocalLength;
float2 ImageCenter;
float K1;
float K2;
float K3;
float K4;
float K5;
float K6;
float P1;
float P2;

float2 BrownConradyUDDistort(float2 InputUV)
{
	// @todo Distortion offset parameter to replace the 0.5 at the beginning and at the end.

	float input_offset_x = bIsUD ? 0.5 : ImageCenter.x;
	float input_offset_y = bIsUD ? 0.5 : ImageCenter.y;
	
	float x_prime = (InputUV.x - input_offset_x) / FocalLength.x;
	float y_prime = (InputUV.y - input_offset_y) / FocalLength.y;

	float x_prime2 = x_prime * x_prime;
	float y_prime2 = y_prime * y_prime;

	float r2 = x_prime2 + y_prime2;
	float r4 = r2 * r2;
	float r6 = r2 * r2 * r2;

	// Compute radial distortion using polynomial division model
	float dr_numerator = 1 + K1 * r2 + K2 * r4 + K3 * r6;
	float dr_denominator = 1 + K4 * r2 + K5 * r4 + K6 * r6;
	float dr = (abs(dr_denominator) < 0.00001) ? dr_numerator : (dr_numerator / dr_denominator);

	// Compute tangential distortion
	float dtx = 2 * P1 * x_prime * y_prime + P2 * (r2 + 2 * x_prime2);
	float dty = P1 * (r2 + 2 * y_prime2) + 2 * P2 * x_prime * y_prime;

	// Compute total distortion
	float x_double_prime = (x_prime * dr) + dtx;
	float y_double_prime = (y_prime * dr) + dty;

	float output_offset_x = bIsUD ? ImageCenter.x : 0.5;
	float output_offset_y = bIsUD ? ImageCenter.y : 0.5;

	float x_undistort = (FocalLength.x * x_double_prime) + output_offset_x;
	float y_undistort = (FocalLength.y * y_double_prime) + output_offset_y;

	float2 result;
	result.x = x_undistort;
	result.y = y_undistort;

	return result;
}

float2 Distort(float2 InputUV)
{
	return BrownConradyUDDistort(InputUV);
}

bool IsDistortionWellOriented(float2 InputUV, float2 DistortedUV, float2 UVOffset)
{
	float2 DistortedXOffset = Distort(InputUV + UVOffset * int2(1, 0));
	float2 DistortedYOffset = Distort(InputUV + UVOffset * int2(0, 1));

	float2 XOffset = DistortedXOffset - DistortedUV;
	float2 YOffset = DistortedYOffset - DistortedUV;
	
	bool bIsFrontFacing = (XOffset.x * YOffset.y - XOffset.y * YOffset.x) >= 0;
	bool bIsRightSideUp = (XOffset.x * UVOffset.x) >= 0 && (YOffset.y * UVOffset.y) >= 0;
	return bIsFrontFacing && bIsRightSideUp;
}

float2 TryDistort(float2 InputUV)
{
	float2 DistortedPosition = Distort(InputUV);
	float2 UVOffset = 0.5 * ThreadIdToUV;
	bool bIsOrientedWell =
		IsDistortionWellOriented(InputUV, DistortedPosition, UVOffset) &&
		IsDistortionWellOriented(InputUV, DistortedPosition, -UVOffset);

	return lerp(InputUV, DistortedPosition, bIsOrientedWell);
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Add half-pixel offset to ThreadID to get accurate UV position
	float2 UndistortedUV = (DispatchThreadId.xy + 0.5) * ThreadIdToUV;
	
	// Rescale the UV by an overscan factor (>= 1.0) to widen the range of UVs to be distorted
	UndistortedUV = ((UndistortedUV - 0.5) * InverseOverscan * CameraOverscan) + 0.5;
	
	float2 DistortedUV = TryDistort(UndistortedUV);
	
	// Compute the UV displacement and scale it by the CameraOverscan to account for the increased resolution of the overscanned scene texture 
	OutDistortionMap[DispatchThreadId.xy] = (DistortedUV - UndistortedUV) / CameraOverscan;
}