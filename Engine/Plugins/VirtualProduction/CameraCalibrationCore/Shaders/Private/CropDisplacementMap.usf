// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CropDisplacementMap.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

Texture2D<float2> InDistortionMapWithOverscan;
RWTexture2D<float2> OutDistortionMap;

// 2D offset from the top left of the overscan displacement map
int2 OverscanOffset;

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 OverscanIndex = DispatchThreadId.xy + OverscanOffset;
	OutDistortionMap[DispatchThreadId.xy] = InDistortionMapWithOverscan[OverscanIndex];
}
