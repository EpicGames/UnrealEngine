// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CrosshairOverlay.usf: 
=============================================================================*/

#include "/Engine/Public/Platform.ush"

// Normalized image center of the lens where the crosshair will be centered
float2 PrincipalPoint;

// Size of the output texture in (pixels)
uint2 OutputTextureExtent;

// Length of one side of the crosshair (in pixels)
float LengthInPixels;

// Size of the gap in the center of the crosshair (in pixels)
float GapSizeInPixels;

// Color to draw the crosshair
float4 OutputColor;

bool IsEqualCoordinate(float A, float B)
{
	// Pixel coordinates are considered equal if they are within a half-pixel of each other
	if (abs(A - B) < 0.5)
	{
		return true;
	}
	return false;
}

void MainPS(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 PixelCoordinate = SvPosition.xy - float2(0.5, 0.5); // Half-pixel shift
    float2 CenterCoordinate = floor(PrincipalPoint * OutputTextureExtent);

	float4 Color = float4(0.0, 0.0, 0.0, 0.0); // Transparent
    
    if (IsEqualCoordinate(PixelCoordinate.x, CenterCoordinate.x) && (
        ((PixelCoordinate.y > (CenterCoordinate.y - LengthInPixels) && PixelCoordinate.y < (CenterCoordinate.y - GapSizeInPixels))) || 
        ((PixelCoordinate.y < (CenterCoordinate.y + LengthInPixels)) && (PixelCoordinate.y > (CenterCoordinate.y + GapSizeInPixels)))))
    {
		Color = OutputColor;
    }   
    else if (IsEqualCoordinate(PixelCoordinate.y, CenterCoordinate.y) && (
        ((PixelCoordinate.x > (CenterCoordinate.x - LengthInPixels) && PixelCoordinate.x < (CenterCoordinate.x - GapSizeInPixels))) || 
        ((PixelCoordinate.x < (CenterCoordinate.x + LengthInPixels)) && (PixelCoordinate.x > (CenterCoordinate.x + GapSizeInPixels)))))
    {
		Color = OutputColor;
    }

    OutColor = Color;
}
