// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionSTMapProcessor.usf
=============================================================================*/

#include "/Engine/Private/Common.ush"


float EvalZoom;
float EvalFocus;

// Stores the x and y values and the x and y tangents for each corner of the blending patch
// Ordered (X0, Y0) -> (X1, Y0) -> (X1, Y1) -> (X0, Y1)
float4 PatchCorners[4];

float2 FxFyScale;
float2 PrincipalPoint;
uint2 OutputTextureExtent;
Texture2D SourceTextureOne;
Texture2D SourceTextureTwo;
Texture2D SourceTextureThree;
Texture2D SourceTextureFour;
SamplerState SourceTextureSampler;

float4 BezierInterp(float4 P0, float4 P1, float4 P2, float4 P3, float Alpha)
{
	float4 P01 = lerp(P0, P1, Alpha);
	float4 P12 = lerp(P1, P2, Alpha);
	float4 P23 = lerp(P2, P3, Alpha);
	float4 P012 = lerp(P01, P12, Alpha);
	float4 P123 = lerp(P12, P23, Alpha);
	float4 P0123 = lerp(P012, P123, Alpha);

	return P0123;
}

float4 EvalAtTwoPoints(float EvalTime, float Time0, float Time1, float4 Value0, float4 Value1, float Tangent0, float Tangent1)
{
	if (Time0 == Time1)
	{
		return Value0;
	}

	float OneThird = 1.0f / 3.0f;

	float Diff = Time1 - Time0;
	float Alpha = (EvalTime - Time0) / Diff;

	float4 DeltaValue = Value1 - Value0;
	float4 TangentScale = DeltaValue / Diff;
	float4 ScaledTangent0 = Tangent0 * TangentScale;
	float4 ScaledTangent1 = Tangent1 * TangentScale;

	float4 P0 = Value0;
	float4 P3 = Value1;
	float4 P1 = P0 + (ScaledTangent0 * Diff * OneThird);
	float4 P2 = P3 - (ScaledTangent1 * Diff * OneThird);

	return BezierInterp(P0, P1, P2, P3, Alpha);
}

float4 CoonsPatchBlend(float X, float Y, float4 Values[4])
{
	float Alpha = 0.0;
	if (PatchCorners[0].x != PatchCorners[1].x)
	{
		Alpha = (X - PatchCorners[0].x) / (PatchCorners[1].x - PatchCorners[0].x);
	}
	
	float Beta = 0.0;
	if (PatchCorners[0].y != PatchCorners[3].y)
	{
		Beta = (Y - PatchCorners[0].y) / (PatchCorners[3].y - PatchCorners[0].y);
	}
	
	float4 X0Curve = EvalAtTwoPoints(X, PatchCorners[0].x, PatchCorners[1].x, Values[0], Values[1], PatchCorners[0].z, PatchCorners[1].z);
	float4 X1Curve = EvalAtTwoPoints(X, PatchCorners[3].x, PatchCorners[2].x, Values[3], Values[2], PatchCorners[3].z, PatchCorners[2].z);
	float4 Y0Curve = EvalAtTwoPoints(Y, PatchCorners[0].y, PatchCorners[3].y, Values[0], Values[3], PatchCorners[0].w, PatchCorners[3].w);
	float4 Y1Curve = EvalAtTwoPoints(Y, PatchCorners[1].y, PatchCorners[2].y, Values[1], Values[2], PatchCorners[1].w, PatchCorners[2].w);
	
	float4 Lx = lerp(X0Curve, X1Curve, Beta);
	float4 Ly = lerp(Y0Curve, Y1Curve, Alpha);
	float4 B = lerp(lerp(Values[0], Values[1], Alpha), lerp(Values[3], Values[2], Alpha), Beta);

	return Lx + Ly - B;
}

void BlendPS(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = SvPosition.xy / OutputTextureExtent.xy;
	UV = ((UV - 0.5f) * FxFyScale) + 0.5f;

	float4 TexOneColor = Texture2DSampleLevel(SourceTextureOne, SourceTextureSampler, UV, 0);
	float4 BlendedColor;

// Case - 2 Focus Curves, 1 Zoom Point Each
#if BLEND_TYPE == 1
	
	float4 TexTwoColor = Texture2DSampleLevel(SourceTextureTwo, SourceTextureSampler, UV, 0);
	BlendedColor = EvalAtTwoPoints(EvalFocus, PatchCorners[0].y, PatchCorners[1].y, TexOneColor, TexTwoColor, PatchCorners[0].w, PatchCorners[1].w);
	
// Case - 1 Focus Curve, 2 Zoom Points
#elif BLEND_TYPE == 2

	float4 TexTwoColor = Texture2DSampleLevel(SourceTextureTwo, SourceTextureSampler, UV, 0);
	BlendedColor = EvalAtTwoPoints(EvalZoom, PatchCorners[0].x, PatchCorners[1].x, TexOneColor, TexTwoColor, PatchCorners[0].z, PatchCorners[1].z);
	
// Case - 2 Focus Curves, 2 Zoom Points Each
#elif BLEND_TYPE == 3

	float4 TexColors[4] =
	{
		TexOneColor,
		Texture2DSampleLevel(SourceTextureTwo, SourceTextureSampler, UV, 0),
		Texture2DSampleLevel(SourceTextureThree, SourceTextureSampler, UV, 0),
		Texture2DSampleLevel(SourceTextureFour, SourceTextureSampler, UV, 0)
	};
	
	BlendedColor = CoonsPatchBlend(EvalZoom, EvalFocus, TexColors);
	
#else
	
	BlendedColor = TexOneColor;

#endif
	
	float2 CenterShift = PrincipalPoint - float2(0.5f, 0.5f);
	OutColor = BlendedColor + float4(CenterShift.xy, CenterShift.xy);
}
