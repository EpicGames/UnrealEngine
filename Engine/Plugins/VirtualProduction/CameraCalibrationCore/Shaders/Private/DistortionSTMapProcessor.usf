// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionSTMapProcessor.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

float2 TexelSize;
uint2 TextureSize;
Texture2D DistortionSTMap;
SamplerState DistortionSTMapSampler;

RWBuffer<float2> OutDistortedUV; 
RWTexture2D<float2> OutUndistortionDisplacementMap;
RWTexture2D<float2> OutDistortionDisplacementMap;

// Must match ECalibratedMapPixelOrigin
#define PIXEL_ORIGIN_TOPLEFT 0
#define PIXEL_ORIGIN_BOTTOMLEFT 1

// Must match ECalibratedMapChannels
#define DISTORTION_CHANNELS_RG 0
#define DISTORTION_CHANNELS_BA 1
#define DISTORTION_CHANNELS_NONE 2

// Must match ECalibratedMapChannels
#define UNDISTORTION_CHANNELS_RG 0
#define UNDISTORTION_CHANNELS_BA 1
#define UNDISTORTION_CHANNELS_NONE 2

[numthreads(8,8,1)]
void DistortionSTMapProcessorMainCS(uint3 ThreadId : SV_DispatchThreadID)
{
	float2 UV = TexelSize * (ThreadId.xy + 0.5f);
	float4 UVMapping = DistortionSTMap.SampleLevel(DistortionSTMapSampler, UV, 0);

	// If the ST Map is flipped vertically, invert the green and alpha channels
#if PIXEL_ORIGIN == PIXEL_ORIGIN_BOTTOMLEFT
	UVMapping.g = 1 - UVMapping.g;
	UVMapping.a = 1 - UVMapping.a;
#endif

#if UNDISTORTION_CHANNELS == UNDISTORTION_CHANNELS_RG
	float2 UndistortedPixel = UVMapping.rg;
#elif UNDISTORTION_CHANNELS == UNDISTORTION_CHANNELS_BA
	float2 UndistortedPixel = UVMapping.ba;
#elif UNDISTORTION_CHANNELS == UNDISTORTION_CHANNELS_NONE
	float2 UndistortedPixel = UV;
#endif

#if DISTORTION_CHANNELS == DISTORTION_CHANNELS_RG
	float2 DistortedPixel = UVMapping.rg;
#elif DISTORTION_CHANNELS == DISTORTION_CHANNELS_BA
	float2 DistortedPixel = UVMapping.ba;
#elif DISTORTION_CHANNELS == DISTORTION_CHANNELS_NONE
	float2 DistortedPixel = UV;
#endif

	OutUndistortionDisplacementMap[ThreadId.xy] = UndistortedPixel - UV;
	OutDistortionDisplacementMap[ThreadId.xy] = DistortedPixel - UV;

	//Stamp distorted UV for known undistorted points
	if(all(ThreadId.xy == uint2(0,0))) //0.0f, 0.0f
	{
		OutDistortedUV[0] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(TextureSize.x * 0.5f, 0))) //0.5f, 0.0f
	{
		OutDistortedUV[1] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(TextureSize.x - 1, 0))) //1.0f, 0.0f
	{
		OutDistortedUV[2] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(TextureSize.x - 1, TextureSize.y * 0.5f))) //1.0f, 0.5f
	{
		OutDistortedUV[3] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(TextureSize.x - 1, TextureSize.y - 1))) //1.0f, 1.0f
	{
		OutDistortedUV[4] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(TextureSize.x * 0.5f, TextureSize.y - 1))) //0.5f, 1.0f
	{
		OutDistortedUV[5] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(0, TextureSize.y - 1))) //0.0f, 1.0f
	{
		OutDistortedUV[6] = DistortedPixel;
	}
	else if(all(ThreadId.xy == uint2(0, TextureSize.y * 0.5f))) //0.0f, 0.5f
	{
		OutDistortedUV[7] = DistortedPixel;
	}	
}
