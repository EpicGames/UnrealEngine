// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FillSTDisplacementMap.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

uint2 GridDimensions;

float Overscan;
float2 PixelToUV;
float2 PixelToOverscannedUV;

Texture2D UndisplacementMap;
SamplerState UndisplacementMapSampler;

float2 ToOverscannedSpace(float2 UV)
{
	return (UV - float2(0.5, 0.5)) / Overscan + float2(0.5, 0.5);
}

void MainVS(
	in uint VertexId : SV_VertexID,
	out noperspective float4 OutTexCoord : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	// Compute UV from vertex ID and rectangle layout
	float2 UnoverscannedUV = float2(VertexId % (GridDimensions.x + 1), VertexId / (GridDimensions.x + 1)) / float2(GridDimensions) + 0.5 * PixelToUV;
	float2 ViewportUV = ToOverscannedSpace(UnoverscannedUV);
	float2 DisplacedUV = ToOverscannedSpace(UnoverscannedUV + UndisplacementMap.SampleLevel(UndisplacementMapSampler, UnoverscannedUV, 0).xy);
	
	// Write the original position in original and overscanned space to the texture coordinate
	OutTexCoord.xy = ViewportUV;
	OutTexCoord.zw = UnoverscannedUV;

	// Output the distorted vertex in clip space
	OutPosition = float4(((DisplacedUV * 2.0) - 1.0) * int2(1, -1), 0.0, 1.0);
}

void MainPS(
	in noperspective float4 InTexCoord : TEXCOORD0,
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 CurrentUV = SvPosition.xy * PixelToOverscannedUV;
	float2 OriginalUV = InTexCoord.xy;

	OutColor= float4((OriginalUV - CurrentUV), 0, 1);
}