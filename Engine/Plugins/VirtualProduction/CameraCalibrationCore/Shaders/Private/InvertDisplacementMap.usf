// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	InvertDisplacementMap.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

uint2 GridDimensions;

float2 PixelToUV;
float2 PixelToOverscanUV;

float OverscanFactor;

Texture2D<float2> DistortionMap;
SamplerState DistortionMapSampler;

void MainVS(
	in uint VertexId : SV_VertexID,
	out noperspective float4 OutTexCoord : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	// Compute UV from vertex ID and rectangle layout
	float2 ViewportUV = float2(VertexId % (GridDimensions.x + 1), VertexId / (GridDimensions.x + 1)) / float2(GridDimensions);

	// Add half-pixel offset to get accurate UV position
	ViewportUV += PixelToOverscanUV * 0.5;
	
	// Output the undistorted viewport UV, rescaled to account for the overscan of the distortion map
	OutTexCoord.xy = ((ViewportUV.xy - 0.5) * OverscanFactor) + 0.5;
	
	float2 Displacement = DistortionMap.SampleLevel(DistortionMapSampler, ViewportUV, 0);
	OutTexCoord.zw = Displacement; // Unused but useful to have this data in a graphics debugger

	// Distort the location of the vertex, then rescale to [-1, 1] for clip space, and invert y
	OutPosition.xy = OutTexCoord.xy + Displacement;
	OutPosition.xy = ((OutPosition.xy * 2.0) - 1.0) * int2(1, -1);

	OutPosition.zw = float2(0.0, 1.0);
}


void MainPS(
	in noperspective float4 InTexCoord : TEXCOORD0,
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 DistortedUV = SvPosition.xy * PixelToUV;
	float2 UndistortedDisplacement = InTexCoord.xy - DistortedUV;

	OutColor = float4(UndistortedDisplacement, 0, 1);
}
