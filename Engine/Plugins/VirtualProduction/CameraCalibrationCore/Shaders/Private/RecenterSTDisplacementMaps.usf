// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Output)

float Overscan;
float2 STMapInvSize;
Texture2D<float2> STMap;
SamplerState STMapSampler;

bool ShouldCrop(float2 UV)
{
	// Add a small border of pixels beyond the normal crop to ensure the boundary doesn't show up in TSR
	float2 OverscannedInvSize = Overscan * STMapInvSize;
	return UV.x + 2 * OverscannedInvSize.x < 0.0 || UV.x - 2 * OverscannedInvSize.x > 1.0 || UV.y + 2 * OverscannedInvSize.y < 0.0 || UV.y - 2 * OverscannedInvSize.y > 1.0;
}

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UnoverscannedUV = (UVAndScreenPos.xy - float2(0.5, 0.5)) * Overscan + float2(0.5, 0.5);
	bool bCropPixel = ShouldCrop(UnoverscannedUV);
	float2 Displacement = STMap.SampleLevel(STMapSampler, UnoverscannedUV, 0);
	OutColor = float4(select(bCropPixel, float2(0.0, 0.0), Displacement / Overscan), 0, 1);
}