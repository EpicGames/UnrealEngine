// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UndistortImagePoints.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

Texture2D DistortionMap;
SamplerState DistortionMapSampler;

StructuredBuffer<float2> InputPoints;
RWBuffer<float2> UndistortedPoints; 

[numthreads(1,1,1)]
void UndistortImagePointsCS(int3 Index : SV_GroupID)
{
	float2 DistortedImagePoint = InputPoints[Index.x];
	float2 Displacement = DistortionMap.SampleLevel(DistortionMapSampler, DistortedImagePoint, 0).xy;
	float2 UndistortedImagePoint = DistortedImagePoint + Displacement;
	UndistortedPoints[Index.x] = UndistortedImagePoint;
}
