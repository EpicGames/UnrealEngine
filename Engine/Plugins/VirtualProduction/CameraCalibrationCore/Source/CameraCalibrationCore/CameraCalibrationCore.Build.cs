// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CameraCalibrationCore : ModuleRules
	{
		public CameraCalibrationCore(ReadOnlyTargetRules Target) : base(Target)
		{
			// Note: For now, avoid adding OpenCVHelper/OpenCV as a dependency until a server dependency issue is resolved.

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"GameplayCameras",
					"RenderCore",
					"RHI",
					"Slate",
					"SlateCore",
					"ProceduralMeshComponent",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
                {
					"Json",
					"LiveLinkInterface",
                    "Projects",
					"Renderer",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			PrivateIncludePaths.AddRange(
				new string[] 
				{
				}
			);
		}
	}
}
