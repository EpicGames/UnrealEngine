// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnamorphicLensDistortionModelHandler.h"

#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnamorphicLensDistortionModelHandler)

// Anamorphic distortion shader parameter structure
BEGIN_SHADER_PARAMETER_STRUCT(FAnamorphicDistortionParams, )
	SHADER_PARAMETER(float, PixelAspect)
	SHADER_PARAMETER(FVector2f, FilmbackSize)
	SHADER_PARAMETER(FVector2f, Squeeze)
	SHADER_PARAMETER(float, LensRotation)
	SHADER_PARAMETER(float, CX02)
	SHADER_PARAMETER(float, CX04)
	SHADER_PARAMETER(float, CX22)
	SHADER_PARAMETER(float, CX24)
	SHADER_PARAMETER(float, CX44)
	SHADER_PARAMETER(float, CY02)
	SHADER_PARAMETER(float, CY04)
	SHADER_PARAMETER(float, CY22)
	SHADER_PARAMETER(float, CY24)
	SHADER_PARAMETER(float, CY44)
END_SHADER_PARAMETER_STRUCT()

// Dedicated anamorphic distortion compute shader
class FAnamorphicDistortionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAnamorphicDistortionCS);
	SHADER_USE_PARAMETER_STRUCT(FAnamorphicDistortionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, ThreadIdToUV)
		SHADER_PARAMETER(float, InverseOverscan)
		SHADER_PARAMETER(float, CameraOverscan)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDistortionMap)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAnamorphicDistortionParams, AnamorphicDistortionParams)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

IMPLEMENT_GLOBAL_SHADER(FAnamorphicDistortionCS, "/Plugin/CameraCalibrationCore/Private/AnamorphicDistortion.usf", "MainCS", SF_Compute);

void UAnamorphicLensDistortionModelHandler::InitializeHandler()
{
	LensModelClass = UAnamorphicLensModel::StaticClass();
}

FVector2D UAnamorphicLensDistortionModelHandler::ComputeDistortedUV(const FVector2D& InScreenUV) const
{
	// Pre-compute some values that will be useful later
	const FVector2D FilmbackSize = FVector2D(CameraFilmback.SensorWidth * AnamorphicParameters.PixelAspect, CameraFilmback.SensorHeight);

	// Guard against divide-by-zero errors
	if (FMath::IsNearlyZero(FilmbackSize.X) || FMath::IsNearlyZero(FilmbackSize.Y) || FMath::IsNearlyZero(AnamorphicParameters.PixelAspect))
	{
		return InScreenUV;
	}
	
	// Pre-compute some values that will be useful later
	const double FilmbackRadius = (0.5) * sqrt(FilmbackSize.X * FilmbackSize.X + FilmbackSize.Y * FilmbackSize.Y);

	const double MountAngleRadians = FMath::DegreesToRadians(AnamorphicParameters.LensRotation);
	const double CosMountAngle = FMath::Cos(MountAngleRadians);
	const double SinMountAngle = FMath::Sin(MountAngleRadians);

	// Step 1: Transform input UVs into diagonally-normalized coordinates, based on the width and height of the filmback/image
	FVector2D DNCoordinates = (InScreenUV - 0.5f) * (FilmbackSize / FilmbackRadius);

	// Step 2: Rotate x and y by the inverse of the mounting angle
	FVector2D DNRotatedCoordinates;
	DNRotatedCoordinates.X = (DNCoordinates.X * CosMountAngle) + (DNCoordinates.Y * SinMountAngle);
	DNRotatedCoordinates.Y = (DNCoordinates.Y * CosMountAngle) - (DNCoordinates.X * SinMountAngle);

	// Step 3: Scale the x-coordinate by the inverse of the pixel aspect ratio
	DNRotatedCoordinates.X = DNRotatedCoordinates.X / AnamorphicParameters.PixelAspect;

	FVector2D DNCoordinatesDistorted = DNRotatedCoordinates;

	// Step 4: Compute the anamorphic distortion for x and y, to the 4th degree
	constexpr uint32 NumIterations = 10;
	for(int i = 0; i < NumIterations; i++)
	{
		const double DNRadiusSq = (DNCoordinatesDistorted.X * DNCoordinatesDistorted.X) + (DNCoordinatesDistorted.Y * DNCoordinatesDistorted.Y);
		const double DNRadius4th = DNRadiusSq * DNRadiusSq;
		const double Phi = FMath::Atan2(DNCoordinatesDistorted.Y, DNCoordinatesDistorted.X);

		const double DistX = (1 + (AnamorphicParameters.CX02 * DNRadiusSq) + (AnamorphicParameters.CX04 * DNRadius4th) + (AnamorphicParameters.CX22 * DNRadiusSq * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CX24 * DNRadius4th * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CX44 * DNRadius4th * FMath::Cos(4 * Phi)));
		const double DistY = (1 + (AnamorphicParameters.CY02 * DNRadiusSq) + (AnamorphicParameters.CY04 * DNRadius4th) + (AnamorphicParameters.CY22 * DNRadiusSq * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CY24 * DNRadius4th * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CY44 * DNRadius4th * FMath::Cos(4 * Phi)));

		DNCoordinatesDistorted.X = (DNRotatedCoordinates.X / DistX) / AnamorphicParameters.SqueezeX;
		DNCoordinatesDistorted.Y = (DNRotatedCoordinates.Y / DistY) / AnamorphicParameters.SqueezeY;
	}

	// Step 5: Rotate x and y by the mounting angle
	DNCoordinates.X = (DNCoordinatesDistorted.X * CosMountAngle) - (DNCoordinatesDistorted.Y * SinMountAngle);
	DNCoordinates.Y = (DNCoordinatesDistorted.Y * CosMountAngle) + (DNCoordinatesDistorted.X * SinMountAngle);

	// Step 6: Scale the x-coordinate by the pixel aspect ratio
	DNCoordinates.X = DNCoordinates.X * AnamorphicParameters.PixelAspect;

	// Step 7: Transform diagonally normalized coordinates back into unit coordinates
	const FVector2D DistortedUV = ((DNCoordinates * FilmbackRadius) / FilmbackSize) + 0.5f;
	return DistortedUV;
}

FVector2D UAnamorphicLensDistortionModelHandler::ComputeUndistortedUV(const FVector2D& InScreenUV) const
{
	// The following implementation is based on the 3DE4 Anamorphic - Standard, Degree 4 model for distortion

	// Pre-compute some values that will be useful later
	const FVector2D FilmbackSize = FVector2D(CameraFilmback.SensorWidth * AnamorphicParameters.PixelAspect, CameraFilmback.SensorHeight);

	// Guard against divide-by-zero errors
	if (FMath::IsNearlyZero(FilmbackSize.X) || FMath::IsNearlyZero(FilmbackSize.Y) || FMath::IsNearlyZero(AnamorphicParameters.PixelAspect))
	{
		return InScreenUV;
	}

	const double FilmbackRadius = 0.5f * FMath::Sqrt((FilmbackSize.X * FilmbackSize.X) + (FilmbackSize.Y * FilmbackSize.Y));

	const double MountAngleRadians = FMath::DegreesToRadians(AnamorphicParameters.LensRotation);
	const double CosMountAngle = FMath::Cos(MountAngleRadians);
	const double SinMountAngle = FMath::Sin(MountAngleRadians);

	// Step 1: Transform input UVs into diagonally-normalized coordinates, based on the width and height of the filmback/image
	FVector2D DNCoordinates = (InScreenUV - 0.5f) * (FilmbackSize / FilmbackRadius);

	// Step 2: Scale the x-coordinate by the inverse of the pixel aspect ratio
	DNCoordinates.X = DNCoordinates.X / AnamorphicParameters.PixelAspect;

	// Step 3: Rotate x and y by the inverse of the mounting angle
	FVector2D DNRotatedCoordinates;
	DNRotatedCoordinates.X = (DNCoordinates.X * CosMountAngle) + (DNCoordinates.Y * SinMountAngle);
	DNRotatedCoordinates.Y = (DNCoordinates.Y * CosMountAngle) - (DNCoordinates.X * SinMountAngle);

	// Step 4: Compute the anamorphic distortion for x and y, to the 4th degree
	const double DNRadiusSq = (DNRotatedCoordinates.X * DNRotatedCoordinates.X) + (DNRotatedCoordinates.Y * DNRotatedCoordinates.Y);
	const double DNRadius4th = DNRadiusSq * DNRadiusSq;
	const double Phi = FMath::Atan2(DNRotatedCoordinates.Y, DNRotatedCoordinates.X);

	FVector2D DNCoordinatesDistorted;
	DNCoordinatesDistorted.X = DNRotatedCoordinates.X * (1 + (AnamorphicParameters.CX02 * DNRadiusSq) + (AnamorphicParameters.CX04 * DNRadius4th) + (AnamorphicParameters.CX22 * DNRadiusSq * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CX24 * DNRadius4th * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CX44 * DNRadius4th * FMath::Cos(4 * Phi)));
	DNCoordinatesDistorted.Y = DNRotatedCoordinates.Y * (1 + (AnamorphicParameters.CY02 * DNRadiusSq) + (AnamorphicParameters.CY04 * DNRadius4th) + (AnamorphicParameters.CY22 * DNRadiusSq * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CY24 * DNRadius4th * FMath::Cos(2 * Phi)) + (AnamorphicParameters.CY44 * DNRadius4th * FMath::Cos(4 * Phi)));

	// Step 5: Scale the x-coordinate by the pixel aspect ratio, and the x- and y-coordinates by the lens-breathing squeeze factors
	DNCoordinatesDistorted.X = DNCoordinatesDistorted.X * AnamorphicParameters.PixelAspect * AnamorphicParameters.SqueezeX;
	DNCoordinatesDistorted.Y = DNCoordinatesDistorted.Y * AnamorphicParameters.SqueezeY;

	// Step 6: Rotate x and y by the mounting angle
	DNCoordinates.X = (DNCoordinatesDistorted.X * CosMountAngle) - (DNCoordinatesDistorted.Y * SinMountAngle);
	DNCoordinates.Y = (DNCoordinatesDistorted.Y * CosMountAngle) + (DNCoordinatesDistorted.X * SinMountAngle);

	// Step 7: Transform diagonally normalized coordinates back into unit coordinates
	const FVector2D DistortedUV = ((DNCoordinates * FilmbackRadius) / FilmbackSize) + 0.5f;

	return DistortedUV;
}

void UAnamorphicLensDistortionModelHandler::InitDistortionMaterials()
{
	if (DistortionPostProcessMID == nullptr)
	{
		UMaterialInterface* DistortionMaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionMaterial(this->StaticClass());
		DistortionPostProcessMID = UMaterialInstanceDynamic::Create(DistortionMaterialParent, this);
	}

	if (UndistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultUndistortionDisplacementMaterial(this->StaticClass());
		UndistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	if (DistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionDisplacementMaterial(this->StaticClass());
		DistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	DistortionPostProcessMID->SetTextureParameterValue("UndistortionDisplacementMap", UndistortionDisplacementMapRT);
	DistortionPostProcessMID->SetTextureParameterValue("DistortionDisplacementMap", DistortionDisplacementMapRT);

	SetDistortionState(CurrentState);
}

void UAnamorphicLensDistortionModelHandler::UpdateMaterialParameters()
{
	//Helper function to set material parameters of an MID
	const auto SetDistortionMaterialParameters = [this](UMaterialInstanceDynamic* const MID)
	{
		MID->SetScalarParameterValue("pixel_aspect", AnamorphicParameters.PixelAspect);

 		MID->SetScalarParameterValue("w_fb", CameraFilmback.SensorWidth * AnamorphicParameters.PixelAspect);
 		MID->SetScalarParameterValue("h_fb", CameraFilmback.SensorHeight);

		MID->SetScalarParameterValue("cx02", AnamorphicParameters.CX02);
		MID->SetScalarParameterValue("cx04", AnamorphicParameters.CX04);
		MID->SetScalarParameterValue("cx22", AnamorphicParameters.CX22);
		MID->SetScalarParameterValue("cx24", AnamorphicParameters.CX24);
		MID->SetScalarParameterValue("cx44", AnamorphicParameters.CX44);

		MID->SetScalarParameterValue("cy02", AnamorphicParameters.CY02);
		MID->SetScalarParameterValue("cy04", AnamorphicParameters.CY04);
		MID->SetScalarParameterValue("cy22", AnamorphicParameters.CY22);
		MID->SetScalarParameterValue("cy24", AnamorphicParameters.CY24);
		MID->SetScalarParameterValue("cy44", AnamorphicParameters.CY44);

		MID->SetScalarParameterValue("sx", AnamorphicParameters.SqueezeX);
		MID->SetScalarParameterValue("sy", AnamorphicParameters.SqueezeY);

		MID->SetScalarParameterValue("phi_mnt", AnamorphicParameters.LensRotation);
	};

	SetDistortionMaterialParameters(UndistortionDisplacementMapMID);
	SetDistortionMaterialParameters(DistortionDisplacementMapMID);
}

void UAnamorphicLensDistortionModelHandler::InterpretDistortionParameters()
{
	LensModelClass->GetDefaultObject<ULensModel>()->FromArray<FAnamorphicDistortionParameters>(CurrentState.DistortionInfo.Parameters, AnamorphicParameters);
}

FString UAnamorphicLensDistortionModelHandler::GetDistortionShaderPath() const
{
	return TEXT("/Plugin/CameraCalibrationCore/Private/AnamorphicDistortion.usf");
}


void UAnamorphicLensDistortionModelHandler::ExecuteDistortionShader(FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const
{
	FIntPoint DistortionMapResolution = OutDistortionMap->Desc.Extent;

	// Early out if the distortion map has invalid dimensions.
	if (DistortionMapResolution.X <= 0 || DistortionMapResolution.Y <= 0)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"AnamorphicLensDistortionModelHandler: Skipping distortion shader execution due to invalid distortion map dimensions (%dx%d)", 
			DistortionMapResolution.X, DistortionMapResolution.Y);
		return;
	}

	FAnamorphicDistortionCS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FAnamorphicDistortionCS::FParameters>();
	
	ShaderParams->OutDistortionMap = GraphBuilder.CreateUAV(OutDistortionMap);
	ShaderParams->ThreadIdToUV = FVector2f(1.0f) / FVector2f(DistortionMapResolution);
	ShaderParams->InverseOverscan = InverseOverscan;
	ShaderParams->CameraOverscan = CameraOverscan;

	// Setup anamorphic distortion parameters
	// Anamorphic model requires exactly 14 parameters (PixelAspect, CX02-CX44, CY02-CY44, SqueezeX, SqueezeY, LensRotation)
	if (InCurrentState.DistortionInfo.Parameters.Num() == 14)
	{
		const float PixelAspect = InCurrentState.DistortionInfo.Parameters[0];
		ShaderParams->AnamorphicDistortionParams.FilmbackSize = FVector2f(SensorSize.X * PixelAspect, SensorSize.Y);
		ShaderParams->AnamorphicDistortionParams.PixelAspect = PixelAspect;
		ShaderParams->AnamorphicDistortionParams.CX02 = InCurrentState.DistortionInfo.Parameters[1];
		ShaderParams->AnamorphicDistortionParams.CX04 = InCurrentState.DistortionInfo.Parameters[2];
		ShaderParams->AnamorphicDistortionParams.CX22 = InCurrentState.DistortionInfo.Parameters[3];
		ShaderParams->AnamorphicDistortionParams.CX24 = InCurrentState.DistortionInfo.Parameters[4];
		ShaderParams->AnamorphicDistortionParams.CX44 = InCurrentState.DistortionInfo.Parameters[5];
		ShaderParams->AnamorphicDistortionParams.CY02 = InCurrentState.DistortionInfo.Parameters[6];
		ShaderParams->AnamorphicDistortionParams.CY04 = InCurrentState.DistortionInfo.Parameters[7];
		ShaderParams->AnamorphicDistortionParams.CY22 = InCurrentState.DistortionInfo.Parameters[8];
		ShaderParams->AnamorphicDistortionParams.CY24 = InCurrentState.DistortionInfo.Parameters[9];
		ShaderParams->AnamorphicDistortionParams.CY44 = InCurrentState.DistortionInfo.Parameters[10];
		ShaderParams->AnamorphicDistortionParams.Squeeze = FVector2f(InCurrentState.DistortionInfo.Parameters[11], InCurrentState.DistortionInfo.Parameters[12]);
		ShaderParams->AnamorphicDistortionParams.LensRotation = InCurrentState.DistortionInfo.Parameters[13];
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"AnamorphicLensDistortionModelHandler: Invalid parameter count (%d), expected exactly 14. Using default values.", 
			InCurrentState.DistortionInfo.Parameters.Num()
		);

		ShaderParams->AnamorphicDistortionParams.FilmbackSize = FVector2f(SensorSize.X, SensorSize.Y);
		ShaderParams->AnamorphicDistortionParams.PixelAspect = 1.0f;
		ShaderParams->AnamorphicDistortionParams.CX02 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CX04 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CX22 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CX24 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CX44 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CY02 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CY04 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CY22 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CY24 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.CY44 = 0.0f;
		ShaderParams->AnamorphicDistortionParams.Squeeze = FVector2f(1.0f, 1.0f);
		ShaderParams->AnamorphicDistortionParams.LensRotation = 0.0f;
	}


	TShaderMapRef<FAnamorphicDistortionCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("AnamorphicDistortionDisplacementMap"),
		ComputeShader,
		ShaderParams,
		FIntVector(FMath::DivideAndRoundUp(DistortionMapResolution.X, 8), FMath::DivideAndRoundUp(DistortionMapResolution.Y, 8), 1));
}
