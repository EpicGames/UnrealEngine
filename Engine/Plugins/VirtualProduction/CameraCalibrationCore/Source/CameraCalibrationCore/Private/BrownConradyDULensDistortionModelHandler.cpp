// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrownConradyDULensDistortionModelHandler.h"

#include "CameraCalibrationSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Models/BrownConradyDULensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrownConradyDULensDistortionModelHandler)

void UBrownConradyDULensDistortionModelHandler::InitializeHandler()
{
	// Initialize with DU model instead of UD
	LensModelClass = UBrownConradyDULensModel::StaticClass();
}

FVector2D UBrownConradyDULensDistortionModelHandler::ComputeDistortedUV(const FVector2D& InScreenUV) const
{
	// For DU model, ComputeDistortedUV actually performs undistortion
	// So we call the parent's ComputeUndistortedUV
	return Super::ComputeUndistortedUV(InScreenUV);
}

FVector2D UBrownConradyDULensDistortionModelHandler::ComputeUndistortedUV(const FVector2D& InScreenUV) const
{
	// For DU model, ComputeUndistortedUV actually performs distortion
	// So we call the parent's ComputeDistortedUV
	return Super::ComputeDistortedUV(InScreenUV);
}

void UBrownConradyDULensDistortionModelHandler::InitDistortionMaterials()
{
	// Initialize materials
	if (DistortionPostProcessMID == nullptr)
	{
		UMaterialInterface* DistortionMaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionMaterial(UBrownConradyDULensDistortionModelHandler::StaticClass());
		DistortionPostProcessMID = UMaterialInstanceDynamic::Create(DistortionMaterialParent, this);
	}

	if (UndistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultUndistortionDisplacementMaterial(UBrownConradyDULensDistortionModelHandler::StaticClass());
		UndistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	if (DistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionDisplacementMaterial(UBrownConradyDULensDistortionModelHandler::StaticClass());
		DistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	DistortionPostProcessMID->SetTextureParameterValue("UndistortionDisplacementMap", UndistortionDisplacementMapRT);
	DistortionPostProcessMID->SetTextureParameterValue("DistortionDisplacementMap", DistortionDisplacementMapRT);

	SetDistortionState(CurrentState);
}