// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrownConradyUDLensDistortionModelHandler.h"

#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Math/NumericLimits.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrownConradyUDLensDistortionModelHandler)

// Brown-Conrady U-D distortion shader parameter structure
BEGIN_SHADER_PARAMETER_STRUCT(FBrownConradyUDDistortionParams, )
	SHADER_PARAMETER(FVector2f, FocalLength)
	SHADER_PARAMETER(FVector2f, ImageCenter)
	SHADER_PARAMETER(float, K1)
	SHADER_PARAMETER(float, K2)
	SHADER_PARAMETER(float, K3)
	SHADER_PARAMETER(float, K4)
	SHADER_PARAMETER(float, K5)
	SHADER_PARAMETER(float, K6)
	SHADER_PARAMETER(float, P1)
	SHADER_PARAMETER(float, P2)
END_SHADER_PARAMETER_STRUCT()

// Dedicated Brown-Conrady U-D distortion compute shader
class FBrownConradyUDDistortionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBrownConradyUDDistortionCS);
	SHADER_USE_PARAMETER_STRUCT(FBrownConradyUDDistortionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, ThreadIdToUV)
		SHADER_PARAMETER(float, InverseOverscan)
		SHADER_PARAMETER(float, CameraOverscan)
		SHADER_PARAMETER_STRUCT_INCLUDE(FBrownConradyUDDistortionParams, BrownConradyUDDistortionParams)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDistortionMap)
		SHADER_PARAMETER(int32, bIsUD)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

IMPLEMENT_GLOBAL_SHADER(FBrownConradyUDDistortionCS, "/Plugin/CameraCalibrationCore/Private/BrownConradyUDDistortion.usf", "MainCS", SF_Compute);

void UBrownConradyUDLensDistortionModelHandler::InitializeHandler()
{
	LensModelClass = UBrownConradyUDLensModel::StaticClass();
}

FVector2D UBrownConradyUDLensDistortionModelHandler::ComputeDistortedUV(const FVector2D& InScreenUV) const
{
	const FVector2D FxFy = CurrentState.FocalLengthInfo.FxFy;
	const FVector2D CxCy = CurrentState.ImageCenter.PrincipalPoint;

	if (FMath::IsNearlyZero(FxFy.X) || FMath::IsNearlyZero(FxFy.Y))
	{
		return InScreenUV;
	}

	// @todo distortion offset to replace the 0.5 below
	FVector2D X = (InScreenUV - FVector2D(0.5f, 0.5f)) / FxFy; // normalized coords (undistorted)

	const float R2 = X.X * X.X + X.Y * X.Y;
	const float R4 = R2 * R2;
	const float R6 = R4 * R2;

	const float RadialNumerator = 1.0f
		+ BrownConradyUDParameters.K1 * R2
		+ BrownConradyUDParameters.K2 * R4
		+ BrownConradyUDParameters.K3 * R6;
	
	const float RadialDenominator = 1.0f
		+ BrownConradyUDParameters.K4 * R2
		+ BrownConradyUDParameters.K5 * R4
		+ BrownConradyUDParameters.K6 * R6;
	
	const float Radial = FMath::IsNearlyZero(RadialDenominator, KINDA_SMALL_NUMBER) ? RadialNumerator : (RadialNumerator / RadialDenominator);

	const FVector2D Tangential(
		BrownConradyUDParameters.P2 * (R2 + 2.0f * X.X * X.X) + 2.0f * BrownConradyUDParameters.P1 * X.X * X.Y,
		BrownConradyUDParameters.P1 * (R2 + 2.0f * X.Y * X.Y) + 2.0f * BrownConradyUDParameters.P2 * X.X * X.Y
	);

	X = X * Radial + Tangential;

	return X * FxFy + CxCy; // distorted UV
}

FVector2D UBrownConradyUDLensDistortionModelHandler::ComputeUndistortedUV(const FVector2D& DistortedUV) const
{
	const FVector2D FxFy = CurrentState.FocalLengthInfo.FxFy;
	const FVector2D CxCy = CurrentState.ImageCenter.PrincipalPoint;

	if (FMath::IsNearlyZero(FxFy.X) || FMath::IsNearlyZero(FxFy.Y))
	{
		return DistortedUV;
	}

	const FVector2D Pd = (DistortedUV - CxCy) / FxFy;

	FVector2D P = Pd; // initial guess

	constexpr int32 MaxIters = 10;

	for (int32 It = 0; It < MaxIters; ++It)
	{
		const float R2 = P.X * P.X + P.Y * P.Y;
		const float R4 = R2 * R2;
		const float R6 = R4 * R2;

		const float RadialNumerator = 1.0f
			+ BrownConradyUDParameters.K1 * R2
			+ BrownConradyUDParameters.K2 * R4
			+ BrownConradyUDParameters.K3 * R6;
		
		const float RadialDenominator = 1.0f
			+ BrownConradyUDParameters.K4 * R2
			+ BrownConradyUDParameters.K5 * R4
			+ BrownConradyUDParameters.K6 * R6;
		
		const float Radial = FMath::IsNearlyZero(RadialDenominator, KINDA_SMALL_NUMBER) ? RadialNumerator : (RadialNumerator / RadialDenominator);

		const FVector2D Tangential(
			BrownConradyUDParameters.P2 * (R2 + 2.0f * P.X * P.X) + 2.0f * BrownConradyUDParameters.P1 * P.X * P.Y,
			BrownConradyUDParameters.P1 * (R2 + 2.0f * P.Y * P.Y) + 2.0f * BrownConradyUDParameters.P2 * P.X * P.Y
		);

		const float InvRadial = FMath::IsNearlyZero(Radial, KINDA_SMALL_NUMBER) ? 1.0f : (1.0f / Radial);

		P = (Pd - Tangential) * InvRadial;
	}

	// @todo distortion offset to replace the 0.5 below
	return P * FxFy + FVector2D(0.5f, 0.5f); // undistorted UV
}

void UBrownConradyUDLensDistortionModelHandler::InitDistortionMaterials()
{
	if (DistortionPostProcessMID == nullptr)
	{
		UMaterialInterface* DistortionMaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionMaterial(this->StaticClass());
		DistortionPostProcessMID = UMaterialInstanceDynamic::Create(DistortionMaterialParent, this);
	}

	if (UndistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultUndistortionDisplacementMaterial(this->StaticClass());
		UndistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	if (DistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionDisplacementMaterial(this->StaticClass());
		DistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	DistortionPostProcessMID->SetTextureParameterValue("UndistortionDisplacementMap", UndistortionDisplacementMapRT);
	DistortionPostProcessMID->SetTextureParameterValue("DistortionDisplacementMap", DistortionDisplacementMapRT);

	SetDistortionState(CurrentState);
}

void UBrownConradyUDLensDistortionModelHandler::UpdateMaterialParameters()
{
	//Helper function to set material parameters of an MID
	const auto SetDistortionMaterialParameters = [this](UMaterialInstanceDynamic* const MID)
	{
		MID->SetScalarParameterValue("k1", BrownConradyUDParameters.K1);
		MID->SetScalarParameterValue("k2", BrownConradyUDParameters.K2);
		MID->SetScalarParameterValue("k3", BrownConradyUDParameters.K3);
		MID->SetScalarParameterValue("k4", BrownConradyUDParameters.K4);
		MID->SetScalarParameterValue("k5", BrownConradyUDParameters.K5);
		MID->SetScalarParameterValue("k6", BrownConradyUDParameters.K6);
		MID->SetScalarParameterValue("p1", BrownConradyUDParameters.P1);
		MID->SetScalarParameterValue("p2", BrownConradyUDParameters.P2);

		MID->SetScalarParameterValue("cx", CurrentState.ImageCenter.PrincipalPoint.X);
		MID->SetScalarParameterValue("cy", CurrentState.ImageCenter.PrincipalPoint.Y);

		MID->SetScalarParameterValue("fx", CurrentState.FocalLengthInfo.FxFy.X);
		MID->SetScalarParameterValue("fy", CurrentState.FocalLengthInfo.FxFy.Y);
	};

	SetDistortionMaterialParameters(UndistortionDisplacementMapMID);
	SetDistortionMaterialParameters(DistortionDisplacementMapMID);
}

void UBrownConradyUDLensDistortionModelHandler::InterpretDistortionParameters()
{
	LensModelClass->GetDefaultObject<ULensModel>()->FromArray<FBrownConradyUDDistortionParameters>(CurrentState.DistortionInfo.Parameters, BrownConradyUDParameters);
}

FString UBrownConradyUDLensDistortionModelHandler::GetDistortionShaderPath() const
{
	return TEXT("/Plugin/CameraCalibrationCore/Private/BrownConradyUDDistortion.usf");
}


void UBrownConradyUDLensDistortionModelHandler::ExecuteDistortionShader(FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const
{
	FIntPoint DistortionMapResolution = OutDistortionMap->Desc.Extent;

	// Early out if the distortion map has invalid dimensions.
	if (DistortionMapResolution.X <= 0 || DistortionMapResolution.Y <= 0)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"BrownConradyUDLensDistortionModelHandler: Skipping distortion shader execution due to invalid distortion map dimensions (%dx%d)", 
			DistortionMapResolution.X, DistortionMapResolution.Y);
		return;
	}

	FBrownConradyUDDistortionCS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FBrownConradyUDDistortionCS::FParameters>();
	
	ShaderParams->OutDistortionMap = GraphBuilder.CreateUAV(OutDistortionMap);
	ShaderParams->ThreadIdToUV = FVector2f(1.0f) / FVector2f(DistortionMapResolution);
	ShaderParams->InverseOverscan = InverseOverscan;
	ShaderParams->CameraOverscan = CameraOverscan;

	// Setup Brown-Conrady U-D distortion parameters

	ShaderParams->BrownConradyUDDistortionParams.ImageCenter = FVector2f(InCurrentState.ImageCenter.PrincipalPoint);
	ShaderParams->BrownConradyUDDistortionParams.FocalLength = FVector2f(InCurrentState.FocalLengthInfo.FxFy);
	ShaderParams->bIsUD = (GetClass() == UBrownConradyUDLensDistortionModelHandler::StaticClass());
	
	// Brown-Conrady U-D model requires exactly 8 parameters (K1, K2, K3, K4, K5, K6, P1, P2)
	if (InCurrentState.DistortionInfo.Parameters.Num() == 8)
	{
		ShaderParams->BrownConradyUDDistortionParams.K1 = InCurrentState.DistortionInfo.Parameters[0];
		ShaderParams->BrownConradyUDDistortionParams.K2 = InCurrentState.DistortionInfo.Parameters[1];
		ShaderParams->BrownConradyUDDistortionParams.K3 = InCurrentState.DistortionInfo.Parameters[2];
		ShaderParams->BrownConradyUDDistortionParams.K4 = InCurrentState.DistortionInfo.Parameters[3];
		ShaderParams->BrownConradyUDDistortionParams.K5 = InCurrentState.DistortionInfo.Parameters[4];
		ShaderParams->BrownConradyUDDistortionParams.K6 = InCurrentState.DistortionInfo.Parameters[5];
		ShaderParams->BrownConradyUDDistortionParams.P1 = InCurrentState.DistortionInfo.Parameters[6];
		ShaderParams->BrownConradyUDDistortionParams.P2 = InCurrentState.DistortionInfo.Parameters[7];
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"BrownConradyUDLensDistortionModelHandler: Invalid parameter count (%d), expected exactly 8. Using default values.", 
			InCurrentState.DistortionInfo.Parameters.Num()
		);

		ShaderParams->BrownConradyUDDistortionParams.K1 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.K2 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.K3 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.K4 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.K5 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.K6 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.P1 = 0.0f;
		ShaderParams->BrownConradyUDDistortionParams.P2 = 0.0f;
	}

	TShaderMapRef<FBrownConradyUDDistortionCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("BrownConradyUDDistortionDisplacementMap"),
		ComputeShader,
		ShaderParams,
		FIntVector(FMath::DivideAndRoundUp(DistortionMapResolution.X, 8), FMath::DivideAndRoundUp(DistortionMapResolution.Y, 8), 1));
}