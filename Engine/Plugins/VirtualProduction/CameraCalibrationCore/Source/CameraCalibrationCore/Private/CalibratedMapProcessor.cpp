// Copyright Epic Games, Inc. All Rights Reserved.

#include "CalibratedMapProcessor.h"

#include "CameraCalibrationCoreLog.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"		
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIGPUReadback.h"

class FCalibratedMapDerivedDataCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCalibratedMapDerivedDataCS);

	class FMapPixelOrigin : SHADER_PERMUTATION_ENUM_CLASS("PIXEL_ORIGIN", ECalibratedMapPixelOrigin);
	class FUndistortionChannels : SHADER_PERMUTATION_ENUM_CLASS("UNDISTORTION_CHANNELS", ECalibratedMapChannels);
	class FDistortionChannels : SHADER_PERMUTATION_ENUM_CLASS("DISTORTION_CHANNELS", ECalibratedMapChannels);

	using FPermutationDomain = TShaderPermutationDomain<FMapPixelOrigin, FUndistortionChannels, FDistortionChannels>;

	SHADER_USE_PARAMETER_STRUCT(FCalibratedMapDerivedDataCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER(FIntPoint, TextureSize)
		
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortionSTMap)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortionSTMapSampler)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<FVector2f>, OutDistortedUV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, OutUndistortionDisplacementMap)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, OutDistortionDisplacementMap)
		END_SHADER_PARAMETER_STRUCT()

public:
	// Called by the engine to determine which permutations to compile for this shader
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

 // --------------------------------------------------------------------------------------------------------------------
IMPLEMENT_GLOBAL_SHADER(FCalibratedMapDerivedDataCS, "/Plugin/CameraCalibrationCore/Private/DistortionSTMapProcessor.usf", "DistortionSTMapProcessorMainCS", SF_Compute);
							 

FCalibratedMapProcessor::FCalibratedMapProcessor()
{
	RunningJobs.Reserve(JobCount);
	for(int32 Index = 0; Index < JobCount; ++Index)
	{
		TSharedPtr<FDerivedDistortionDataJob> NewJob = MakeShared<FDerivedDistortionDataJob>();
		NewJob->Readback = MakePimpl<FRHIGPUBufferReadback>(*FString::Printf(TEXT("DerivedDistortionDataJobReadback_%02d"), Index));
		AvailableJobs.Enqueue(NewJob);
	}
}

FCalibratedMapProcessor::~FCalibratedMapProcessor()
{
	FlushRenderingCommands();

	FScopeLock Lock(&RunningJobsCriticalSection);
	RunningJobs.Empty();
	CompletedJobs.Empty();
	AvailableJobs.Empty();
}

void FCalibratedMapProcessor::Update()
{
	//If we have completed jobs, pop them and callback owner
	TSharedPtr<FDerivedDistortionDataJob> CompletedJob = nullptr;
	while (CompletedJobs.Dequeue(CompletedJob))
	{
		CompletedJob->JobArgs.JobCompletedCallback.ExecuteIfBound(CompletedJob->Output);
		
		AvailableJobs.Enqueue(CompletedJob);
	}

	//Fill in available job slots with any pending ones
	while (PendingJobs.Peek())
	{
		//push pending jobs
		TSharedPtr<FDerivedDistortionDataJob> NewJob = nullptr;
		if (AvailableJobs.Dequeue(NewJob))
		{
			//We have a slot available
			FDerivedDistortionDataJobArgs PendingJob;

			//Consume Pending job that was peeked
			verify(PendingJobs.Dequeue(PendingJob));

			NewJob->Output.Reset();
			NewJob->Output.Focus = PendingJob.Focus;
			NewJob->Output.Zoom = PendingJob.Zoom;
			NewJob->JobArgs = MoveTemp(PendingJob);
			ExecuteJob(MoveTemp(NewJob));
		}
		else
		{
			//No more job slots available, exit
			break;
		}
	}

	ENQUEUE_RENDER_COMMAND(FCalibratedMapProcessor_Update_RenderThread)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			Update_RenderThread();
		});
}

bool FCalibratedMapProcessor::PushDerivedDistortionDataJob(FDerivedDistortionDataJobArgs&& JobArgs)
{
	//Validate arguments before pushing the job
	if (!JobArgs.JobCompletedCallback.IsBound()
		|| !JobArgs.SourceDistortionMap.IsValid()
		|| !JobArgs.OutputUndistortionDisplacementMap.IsValid()
		|| !JobArgs.OutputDistortionDisplacementMap.IsValid())
	{
		return false;
	}

	TSharedPtr<FDerivedDistortionDataJob> NewJob = nullptr;
	if (AvailableJobs.Dequeue(NewJob))
	{
		NewJob->Output.Reset();
		NewJob->Output.Focus = JobArgs.Focus;
		NewJob->Output.Zoom = JobArgs.Zoom;
		NewJob->JobArgs = MoveTemp(JobArgs);
		ExecuteJob(NewJob);
	}
	
	if(NewJob == nullptr)
	{
		//No slots available, push that to the pending job
		PendingJobs.Enqueue(MoveTemp(JobArgs));
	}
	
	return true;
}

void FCalibratedMapProcessor::ExecuteJob(TSharedPtr<FDerivedDistortionDataJob> Job)
{
	check(Job);
	{
		//Early validation of inputs in case some textures are now invalid
		UTexture* SourceTexture = Job->JobArgs.SourceDistortionMap.Get();
		UTextureRenderTarget2D* DestinationUndistortionTexture = Job->JobArgs.OutputUndistortionDisplacementMap.Get();
		UTextureRenderTarget2D* DestinationDistortionTexture = Job->JobArgs.OutputDistortionDisplacementMap.Get();
		if(SourceTexture == nullptr
		|| DestinationUndistortionTexture == nullptr
		|| DestinationDistortionTexture == nullptr
		|| SourceTexture->GetResource() == nullptr
		|| DestinationUndistortionTexture->GetResource() == nullptr
		|| DestinationDistortionTexture->GetResource() == nullptr)
		{
			Job->State = EDerivedDistortionDataJobState::Completed;
			Job->Output.Result = EDerivedDistortionDataResult::Error;
			CompletedJobs.Enqueue(Job);
			return;
		}
	
		FTextureResource* SourceDistortionMap = SourceTexture->GetResource();
		FTextureResource* DestinationUndistortionDisplacementMap = DestinationUndistortionTexture->GetResource();
		FTextureResource* DestinationDistortionDisplacementMap = DestinationDistortionTexture->GetResource();

		{
			FScopeLock Lock(&RunningJobsCriticalSection);
			RunningJobs.Add(Job);
		}
		
		ENQUEUE_RENDER_COMMAND(FCalibratedMapProcessor_ComputeDerivedDistortionData)(
		[DestinationUndistortionDisplacementMap, DestinationDistortionDisplacementMap, SourceDistortionMap, Job](FRHICommandListImmediate& RHICmdList)
        {
            FRDGBuilder GraphBuilder(RHICmdList);

            const FRDGTextureRef SourceSTMap = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceDistortionMap->TextureRHI, TEXT("SourceSTMap")));
            const FRDGTextureRef UndistortionDisplacementMap = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationUndistortionDisplacementMap->TextureRHI, TEXT("UndistortionDestinationDisplacementMap")));
            const FRDGTextureRef DistortionDisplacementMap = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationDistortionDisplacementMap->TextureRHI, TEXT("DistortionDestinationDisplacementMap")));

			const int32 EntriesCount = Job->Output.EdgePointCount;
			const int32 ReadbackDataByteCount = Job->Output.EdgePointsDistortedUVs.GetTypeSize();
			FRDGBufferDesc EdgePointsDistortedUVDesc = FRDGBufferDesc::CreateBufferDesc(ReadbackDataByteCount, EntriesCount);
			EdgePointsDistortedUVDesc.Usage = static_cast<EBufferUsageFlags>(EdgePointsDistortedUVDesc.Usage | BUF_SourceCopy);
            const FRDGBufferRef EdgePointsBuffer = GraphBuilder.CreateBuffer(EdgePointsDistortedUVDesc, TEXT("EdgePointsBuffer"));
           	
			FCalibratedMapDerivedDataCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCalibratedMapDerivedDataCS::FParameters>();
            Parameters->DistortionSTMap = SourceSTMap;
           	Parameters->DistortionSTMapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
           	Parameters->TexelSize = FVector2f(1.0f / UndistortionDisplacementMap->Desc.Extent.X, 1.0f / UndistortionDisplacementMap->Desc.Extent.Y);
            Parameters->TextureSize = FIntPoint(UndistortionDisplacementMap->Desc.Extent.X, UndistortionDisplacementMap->Desc.Extent.Y);

            //Create UAVs for compute shader outputs
           	Parameters->OutDistortedUV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(EdgePointsBuffer, PF_G32R32F));
			Parameters->OutUndistortionDisplacementMap = GraphBuilder.CreateUAV(UndistortionDisplacementMap);
			Parameters->OutDistortionDisplacementMap = GraphBuilder.CreateUAV(DistortionDisplacementMap);

			FCalibratedMapDerivedDataCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCalibratedMapDerivedDataCS::FMapPixelOrigin>(Job->JobArgs.Format.PixelOrigin);
			PermutationVector.Set<FCalibratedMapDerivedDataCS::FUndistortionChannels>(Job->JobArgs.Format.UndistortionChannels);
			PermutationVector.Set<FCalibratedMapDerivedDataCS::FDistortionChannels>(Job->JobArgs.Format.DistortionChannels);

            FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FCalibratedMapDerivedDataCS> ComputeShader(GlobalShaderMap, PermutationVector);

            GraphBuilder.AddPass(
            			RDG_EVENT_NAME("LensFileSTMapConversion"),
            			Parameters,
            			ERDGPassFlags::Compute,
            			[Parameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
						{
            				const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(Parameters->TextureSize, FComputeShaderUtils::kGolden2DGroupSize);
            				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, GroupCount);
						}
            );

            //We need to access the data related to points on the edge to compute overscan factor
			AddEnqueueCopyPass(GraphBuilder, Job->Readback.Get(), EdgePointsBuffer, ReadbackDataByteCount * EntriesCount);
            
			//Kick graph
			GraphBuilder.Execute();

			//Once graph is kicked, mark the job as waiting
			Job->State = EDerivedDistortionDataJobState::AwaitingResult;
        });		
	}
}



void FCalibratedMapProcessor::Update_RenderThread()
{
	TArray<TSharedPtr<FDerivedDistortionDataJob>, TInlineAllocator<JobCount>> ToComplete;
	{
		FScopeLock Lock(&RunningJobsCriticalSection);
		for (auto It = RunningJobs.CreateIterator(); It; ++It)
		{
			TSharedPtr<FDerivedDistortionDataJob> CompletedJob = *It;
			if (CompletedJob->State == EDerivedDistortionDataJobState::AwaitingResult)
			{
				if (CompletedJob->Readback->IsReady())
				{
					ToComplete.Add(CompletedJob);
					It.RemoveCurrent();
				}
			}
		}
	}

	//Now process out of critical section completed jobs to add them to the queue
	for(TSharedPtr<FDerivedDistortionDataJob>& CompletedJob : ToComplete)
	{
		constexpr int32 EntriesCount = FDerivedDistortionDataJobOutput::EdgePointCount;
		const int32 ReadbackDataByteCount = CompletedJob->Output.EdgePointsDistortedUVs.GetTypeSize();
		const FVector2f* EdgePointsDistortedUV = static_cast<const FVector2f*>(CompletedJob->Readback->Lock(EntriesCount * ReadbackDataByteCount));

		if (EdgePointsDistortedUV)
		{
			for (int32 Index = 0; Index < CompletedJob->Output.EdgePointsDistortedUVs.Num(); ++Index)
			{
				const FVector2f DistortedUV = EdgePointsDistortedUV[Index];
				CompletedJob->Output.EdgePointsDistortedUVs[Index] = FVector2D(DistortedUV.X, DistortedUV.Y);
				CompletedJob->Output.Result = EDerivedDistortionDataResult::Success;
			}
		}
		else
		{
			UE_LOGF(LogCameraCalibrationCore, Error, "Failed to retrieve edge points distorted UVs");
			CompletedJob->Output.Result = EDerivedDistortionDataResult::Error;
		}

		CompletedJob->Readback->Unlock();

		CompletedJob->State = EDerivedDistortionDataJobState::Completed;
		CompletedJobs.Enqueue(CompletedJob);
	}
}

