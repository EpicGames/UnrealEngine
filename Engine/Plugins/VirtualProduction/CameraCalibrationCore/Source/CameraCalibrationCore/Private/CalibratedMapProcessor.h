// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICalibratedMapProcessor.h"

#include "CalibratedMapFormat.h"
#include "Containers/Queue.h"
#include "Templates/PimplPtr.h"
#include "UObject/WeakObjectPtr.h"

class UTexture;
struct FDerivedDistortionData;
class FRHIGPUBufferReadback;
class UTextureRenderTarget2D;

/** 
 * Result of processing derived data 
 */
enum class EDerivedDistortionDataResult : uint8
{
	Success,
	Error,
};

/**  
 * Arguments for a job to derived data from calibrated map 
 */
struct FDerivedDistortionDataJobArgs
{
	/** Focus point this is associated with */
	float Focus = 0.0f;
	
	/** Zoom point this is associated with */
	float Zoom = 0.0f;

	/** Formating information, such as pixel origin and the channels used for each displacement map */
	FCalibratedMapFormat Format;

	/** Source calibrated map */
	//When coefficient route is supported, we might look at how we can have a mutual interface
	TWeakObjectPtr<UTexture> SourceDistortionMap;

	/** Output derived undistortion displacement map */
	TWeakObjectPtr<UTextureRenderTarget2D> OutputUndistortionDisplacementMap;

	/** Output derived distortion displacement map */
	TWeakObjectPtr<UTextureRenderTarget2D> OutputDistortionDisplacementMap;

	/** Delegate to be called when job is completed */
	FOnDerivedDistortionJobCompleted JobCompletedCallback;
};

/** 
 * Derived data that was generated 
 */
struct FDerivedDistortionDataJobOutput
{
	FDerivedDistortionDataJobOutput()
	{
		Reset();
	}
	
	/** Used when reusing a job */
	void Reset()
	{
		Result = EDerivedDistortionDataResult::Error;
		EdgePointsDistortedUVs.SetNumZeroed(EdgePointCount);
	}
	
	/** Result of that job */
	EDerivedDistortionDataResult Result = EDerivedDistortionDataResult::Error;

	/** Focus point this is associated with */
	float Focus = -1.0f;
	
	/** Zoom point this is associated with */
	float Zoom = -1.0f;

	/** Distorted UVs we found at the 8 edge points */
	static constexpr int32 EdgePointCount = 8;
	TArray<FVector2D> EdgePointsDistortedUVs;
};

/** State of a job */
enum class EDerivedDistortionDataJobState : uint8
{
	Ready,
	AwaitingResult,
	Completed,
};

/**
 * Info for a given job to derive data from calibrated map
 */
struct FDerivedDistortionDataJob
{
	/** State of the job */
	EDerivedDistortionDataJobState State = EDerivedDistortionDataJobState::Ready;
	
	/** Arguments for this job */
	FDerivedDistortionDataJobArgs JobArgs;
	
	/** Manages data we read back from compute shader operation */
	TPimplPtr<FRHIGPUBufferReadback> Readback;

	/** Output generated by this job */
	FDerivedDistortionDataJobOutput Output;
};

/**
 * Processor used to launch job to derived data out of calibrated
 * STMaps/UVMaps. Goal is to have an intermediate data model
 * that we use when we evaluate the LensFile
 */
class FCalibratedMapProcessor : public ICalibratedMapProcessor
{
public:

	FCalibratedMapProcessor();
	virtual ~FCalibratedMapProcessor() override;

	//~ Begin ICalibratedMapProcessor interface
	virtual void Update() override;
	virtual bool PushDerivedDistortionDataJob(FDerivedDistortionDataJobArgs&& JobArgs) override;
	//~ End ICalibratedMapProcessor interface

private:

	/** Execute job by queuing on render thread */
	void ExecuteJob(TSharedPtr<FDerivedDistortionDataJob> Job);
	
	/** Verify if any running jobs are done doing readback and convert to completed */
	void Update_RenderThread();
	
private:

	/** Number of jobs slots to create. Limiting the amount here to limit number of allocation. */
	static constexpr int32 JobCount = 4;
	
	/** Job slots available to be configured and launched */
	TQueue<TSharedPtr<FDerivedDistortionDataJob>> AvailableJobs;

	/** Jobs currently being ran and waiting on gpu readback to be available */
	FCriticalSection RunningJobsCriticalSection;
	TArray<TSharedPtr<FDerivedDistortionDataJob>> RunningJobs;

	/** Jobs that were completed, ready to be dispatched */
	TQueue<TSharedPtr<FDerivedDistortionDataJob>> CompletedJobs;

	/** Jobs that were requested but no slots were available */
	TQueue<FDerivedDistortionDataJobArgs> PendingJobs;
};
