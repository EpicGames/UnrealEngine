// Copyright Epic Games, Inc. All Rights Reserved.

#include "CalibrationPointComponent.h"

#include "Materials/MaterialInterface.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CalibrationPointComponent)

UCalibrationPointComponent::UCalibrationPointComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Disabling collision avoids "Skipping dirty area creation because of empty bounds" warning message.
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
}

void UCalibrationPointComponent::OnRegister()
{
	Super::OnRegister();

	// This one is needed so that property changes are seen when this is a component of a blueprint.
	RebuildVertices();
}

bool UCalibrationPointComponent::GetWorldLocation(const FString& InPointName, FVector& OutLocation) const
{
	FString ComponentName;
	FString SubpointName;

	const bool bWasNamespaced = InPointName.Split(TEXT("::"), &ComponentName, &SubpointName);

	if (bWasNamespaced)
	{
		// If it came namespaced, make sure the root name corresponds to this component's name.
		if (ComponentName != GetName())
		{
			return false;
		}
	}
	else
	{
		// If it wasn't namespaced, it will first try to match the name with the component name.
		if (InPointName == GetName())
		{
			OutLocation = GetComponentLocation();
			return true;
		}

		SubpointName = InPointName;
	}

	for (const auto& Elem : SubPoints)
	{
		if (Elem.Key == SubpointName)
		{
			const FTransform SubPointTransform(FRotator(0, 0, 0), Elem.Value, FVector(1.0f));
			OutLocation = (SubPointTransform * GetComponentTransform()).GetLocation();
			return true;
		}
	}

	return false;
}

bool UCalibrationPointComponent::NamespacedSubpointName(const FString& InSubpointName, FString& OutNamespacedName) const
{
	OutNamespacedName = FString::Printf(TEXT("%s::%s"), *GetName(), *InSubpointName);

	return InSubpointName.Len() > 0;
}

void UCalibrationPointComponent::GetNamespacedPointNames(TArray<FString>& OutNamespacedNames) const
{
	// Component name is the namespace itself
	OutNamespacedNames.Add(GetName());

	for (const auto& Elem : SubPoints)
	{
		FString NamespacedName;
		if (NamespacedSubpointName(Elem.Key, NamespacedName))
		{
			OutNamespacedNames.Add(NamespacedName);
		}
	}
}

#if WITH_EDITOR
void UCalibrationPointComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	RebuildVertices();
}
#endif

void UCalibrationPointComponent::RebuildVertices()
{
	ClearAllMeshSections();

	// Only visualize the points in Editor
	if (!bVisualizePointsInEditor || FApp::IsGame())
	{
		return;
	}

	TArray<FVector> Vertices;
	TArray<int32> Triangles;
	TArray<FLinearColor> VertexColors;

	// Coordinate system
	// 
	// Top View
	// 
	// x ^
	//   |
	//   o--->
	//  z    y
	// 
	// Front View
	// 
	// z ^
	//   |
	//   x--->
	//  x    y
	// 

	auto AddPyramidVerticesForPoint = [&](const FVector& Location, const FTransform Transform)
	{
		// Top view
		//
		//    3
		//   / \
		//  1---2
		//
		// Side view
		//
		//   1---2
		//    \ /
		//     0

		// Vertices

		const float Side = 1.0f;
		const float H = Side * FMath::Sqrt(3.0f) / 2.0f;

		const int32 BaseIndex = Vertices.Num();

		Vertices.Add(FVector(        0,         0,           0)); // 0
		Vertices.Add(FVector(   -H / 3, -Side / 2,  1.5 * Side)); // 1
		Vertices.Add(FVector(   -H / 3,  Side / 2,  1.5 * Side)); // 2
		Vertices.Add(FVector(2 * H / 3,         0,  1.5 * Side)); // 3

		for (int32 VertexIdx = BaseIndex; VertexIdx < Vertices.Num(); ++VertexIdx)
		{
			Vertices[VertexIdx] = Location + PointVisualizationScale * Transform.TransformVector(Vertices[VertexIdx]);
		}

		// Triangles (CCW when looking at face)

		Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 2); Triangles.Add(BaseIndex + 1);
		Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 3); Triangles.Add(BaseIndex + 2);
		Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 1); Triangles.Add(BaseIndex + 3);
		Triangles.Add(BaseIndex + 1); Triangles.Add(BaseIndex + 2); Triangles.Add(BaseIndex + 3);

		// Colors

		const FLinearColor CenterColor (FVector(200, 150, 44) / 255);
		const FLinearColor Vertex1Color(FVector( 32,  76,  9) / 255);
		const FLinearColor Vertex2Color(FVector(  9,  65, 76) / 255);
		const FLinearColor Vertex3Color(FVector( 76,   9, 75) / 255);

		VertexColors.Add(CenterColor);
		VertexColors.Add(Vertex1Color);
		VertexColors.Add(Vertex2Color);
		VertexColors.Add(Vertex3Color);
	};

	auto AddPyramidMeshForPoint = [&](const FVector& Location)
	{
		// This will build a mesh composed of a shape repeated 4 times, each in one of the 4 directions
		// of the vertices of a triangular pyramid, from its center.

		const float H = FMath::Sqrt(3.0f) / 2.0f;

		const FVector Pyramid0(        0,    0,      1);
		const FVector Pyramid1(   -H / 3, -0.5, -H / 2);
		const FVector Pyramid2(   -H / 3,  0.5, -H / 2);
		const FVector Pyramid3(2 * H / 3,    0, -H / 2);

		// The 4 directions
		const TArray<FTransform> Transforms =
		{
			FTransform::Identity,
			FTransform(FQuat::FindBetweenVectors(Pyramid0, Pyramid1)),
			FTransform(FQuat::FindBetweenVectors(Pyramid0, Pyramid2)),
			FTransform(FQuat::FindBetweenVectors(Pyramid0, Pyramid3)),
		};

		// The 4 repetitions of the shape, each with a different orientation
		for (const FTransform& Transform : Transforms)
		{
			AddPyramidVerticesForPoint(Location, Transform);
		}
	};

	auto AddCubeMeshForPoint = [&](const FVector& Location)
	{
		// Left-view   Front-view   Right-view   Back-view
		//
		// 4---0       0---1        1---6        6---4
		// |   |       |   |        |   |        |   |
		// 5---3       3---2        2---7        7---5
		//
		//             Top-view     Bot-view
		//             
		//             4---6        3---2
		//             |   |        |   |
		//             0---1        5---7
		//

		const int32 BaseIndex = Vertices.Num();

		// Vertices

		Vertices.Add(FVector(-1, -1,  1) / 2); // 0
		Vertices.Add(FVector(-1,  1,  1) / 2); // 1
		Vertices.Add(FVector(-1,  1, -1) / 2); // 2
		Vertices.Add(FVector(-1, -1, -1) / 2); // 3
		Vertices.Add(FVector( 1, -1,  1) / 2); // 4
		Vertices.Add(FVector( 1, -1, -1) / 2); // 5
		Vertices.Add(FVector( 1,  1,  1) / 2); // 6
		Vertices.Add(FVector( 1,  1, -1) / 2); // 7

		// Colors
		{
			const FLinearColor Color(FVector(1, 0, 1));

			for (int32 VertexIdx = BaseIndex; VertexIdx < Vertices.Num(); ++VertexIdx)
			{
				VertexColors.Add(Color);
			}
		}

		// Apply scale and translate to point location.
		for (int32 VertexIdx = BaseIndex; VertexIdx < Vertices.Num(); ++VertexIdx)
		{
			Vertices[VertexIdx] = Location + PointVisualizationScale * Vertices[VertexIdx];
		}

		// Triangles (CCW when looking at face)

		// Front
		Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 3); Triangles.Add(BaseIndex + 2);
		Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 2); Triangles.Add(BaseIndex + 1);

		// Left
		Triangles.Add(BaseIndex + 4); Triangles.Add(BaseIndex + 3); Triangles.Add(BaseIndex + 0);
		Triangles.Add(BaseIndex + 4); Triangles.Add(BaseIndex + 5); Triangles.Add(BaseIndex + 3);
		
		// Right
		Triangles.Add(BaseIndex + 1); Triangles.Add(BaseIndex + 7); Triangles.Add(BaseIndex + 6);
		Triangles.Add(BaseIndex + 1); Triangles.Add(BaseIndex + 2); Triangles.Add(BaseIndex + 7);
		
		// Back
		Triangles.Add(BaseIndex + 6); Triangles.Add(BaseIndex + 5); Triangles.Add(BaseIndex + 4);
		Triangles.Add(BaseIndex + 6); Triangles.Add(BaseIndex + 7); Triangles.Add(BaseIndex + 5);

		// Top
		Triangles.Add(BaseIndex + 4); Triangles.Add(BaseIndex + 1); Triangles.Add(BaseIndex + 6);
		Triangles.Add(BaseIndex + 4); Triangles.Add(BaseIndex + 0); Triangles.Add(BaseIndex + 1);
		
		// Bottom
		Triangles.Add(BaseIndex + 3); Triangles.Add(BaseIndex + 7); Triangles.Add(BaseIndex + 2);
		Triangles.Add(BaseIndex + 3); Triangles.Add(BaseIndex + 5); Triangles.Add(BaseIndex + 7);
	};

	auto AddMeshForPoint = [&](const FVector& Location)
	{
		switch (VisualizationShape)
		{
		case CalibrationPointVisualizationCube:
			AddCubeMeshForPoint(Location);
			break;

		case CalibrationPointVisualizationPyramid:
			AddPyramidMeshForPoint(Location);
			break;

		default:
			break;
		}
	};

	// Draw the 1st point, which is the Component location.
	const FVector OriginPoint(0.0f);
	AddMeshForPoint(OriginPoint);

	// Draw the subpoints
	for (const auto& Elem : SubPoints)
	{
		AddMeshForPoint(Elem.Value);
	}

	const int32 SectionIndex = 0;
	const TArray<FVector> Normals;
	const TArray<FVector2D> UV0;
	const TArray<FProcMeshTangent> Tangents;
	const bool bCreateCollision = false;

	CreateMeshSection_LinearColor(SectionIndex, Vertices, Triangles, Normals, UV0, VertexColors, Tangents, bCreateCollision);
	SetMeshSectionVisible(SectionIndex, true);

	// Assign material that uses vertex colors as base color.
	if (!GetMaterial(SectionIndex))
	{
		const FString MaterialPath = TEXT("/CameraCalibrationCore/Materials/M_VertexColors.M_VertexColors");

		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(FSoftObjectPath(MaterialPath).TryLoad()))
		{
			SetMaterial(SectionIndex, MaterialInterface);
		}
	}
}
