// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationCharucoBoard.h"
#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSubsystem.h"
#include "CalibrationPointComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"


ACameraCalibrationCharucoBoard::ACameraCalibrationCharucoBoard()
{
	// Create root component
	Root = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
	SetRootComponent(Root);
	RootComponent->SetVisibility(true);

	// Create board mesh component
	BoardMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("BoardMesh"));
	BoardMeshComponent->SetupAttachment(RootComponent);

	// Create fixed calibration point components
	TopLeft     = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("TopLeft"));
	TopRight    = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("TopRight"));
	BottomLeft  = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("BottomLeft"));
	BottomRight = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("BottomRight"));
	Center      = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("Center"));

	TopLeft    ->SetupAttachment(RootComponent);
	TopRight   ->SetupAttachment(RootComponent);
	BottomLeft ->SetupAttachment(RootComponent);
	BottomRight->SetupAttachment(RootComponent);
	Center     ->SetupAttachment(RootComponent);

	// Set default ArUco dictionary. Smaller dictionaries tend to be easier to recongnize.
	ArucoDictionary = EArucoDictionaryShadow::DICT_4X4_50;

	// Try to find default cube mesh
	static ConstructorHelpers::FObjectFinder<UStaticMesh> CubeMeshFinder(TEXT("/Engine/BasicShapes/Cube"));
	if (CubeMeshFinder.Succeeded())
	{
		BoardMesh = CubeMeshFinder.Object;
	}

	// Find the Charuco board material
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaterialFinder(TEXT("/CameraCalibrationCore/Materials/M_CharucoBoard"));
	if (MaterialFinder.Succeeded())
	{
		BoardMaterial = MaterialFinder.Object;
	}
	else
	{
		// Unexpected fallback

		UE_LOGF(LogCameraCalibrationCore, Error, "Failed to load M_CharucoBoard");

		static ConstructorHelpers::FObjectFinder<UMaterialInterface> FallbackFinder(TEXT("/Engine/BasicShapes/BasicShapeMaterial"));
		if (FallbackFinder.Succeeded())
		{
			BoardMaterial = FallbackFinder.Object;
		}
	}
}

void ACameraCalibrationCharucoBoard::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	Rebuild();
}

#if WITH_EDITOR
void ACameraCalibrationCharucoBoard::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		
		// Rebuild when relevant properties change
		if (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, NumCornerRows) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, NumCornerCols) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, SquareSideLength) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, MarkerSizeRatio) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, ArucoDictionary) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, Thickness) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, BoardMesh) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, BoardMaterial) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCharucoBoard, MaxTextureResolution))
		{
			Rebuild();
		}
	}
}
#endif

void ACameraCalibrationCharucoBoard::Rebuild()
{
	Modify();

	CreateBoardMesh();
	GenerateCharucoTexture();
	UpdateCalibrationPointPositions();
}

void ACameraCalibrationCharucoBoard::ClearGeneratedMeshComponent()
{
	// Remove board mesh component
	if (BoardMeshComponent)
	{
		RemoveInstanceComponent(BoardMeshComponent);
		BoardMeshComponent->DestroyComponent();
		BoardMeshComponent = nullptr;
	}
}

void ACameraCalibrationCharucoBoard::CreateBoardMesh()
{
	if (!BoardMesh || !BoardMeshComponent)
	{
		return;
	}

	// Update the mesh and properties
	BoardMeshComponent->SetStaticMesh(BoardMesh);

	// Calculate board size (NumCorners + 1 squares in each direction)
	float BoardWidth = (NumCornerCols + 1) * SquareSideLength;
	float BoardHeight = (NumCornerRows + 1) * SquareSideLength;

	// Scale and position the mesh so origin is at the outer corner
	BoardMeshComponent->SetRelativeScale3D(FVector(Thickness / 100.0f, BoardWidth / 100.0f, BoardHeight / 100.0f));
	
	// Position board so origin (0,0,0) is at bottom-left outer corner
	// The cube mesh has its origin at center, so offset by half the scaled dimensions  
	BoardMeshComponent->SetRelativeLocation(FVector(0.0f, BoardWidth / 2.0f, BoardHeight / 2.0f));

	// Apply material for pattern display
	if (BoardMaterial)
	{
		DynamicBoardMaterial = UMaterialInstanceDynamic::Create(BoardMaterial, this);
		BoardMeshComponent->SetMaterial(0, DynamicBoardMaterial);
	}
}

void ACameraCalibrationCharucoBoard::GenerateCharucoTexture()
{
	// Setup board configuration (NumCorners + 1 == squares in each direction)

	FCharucoBoardConfigShadow Config;

	Config.SquaresX = NumCornerCols + 1;
	Config.SquaresY = NumCornerRows + 1;
	Config.SquareSize = SquareSideLength;
	Config.MarkerSize = SquareSideLength * MarkerSizeRatio;
	Config.Dictionary = ArucoDictionary;

	// Calculate aspect ratio for texture resolution
	FIntPoint AdjustedResolution(MaxTextureResolution, MaxTextureResolution);
	
	if (Config.SquaresY > 0)
	{
		float AspectRatio = float(Config.SquaresX) / float(Config.SquaresY);
		
		if (AspectRatio >= 1.0f)
		{
			// Wider than tall
			AdjustedResolution.X = MaxTextureResolution;
			AdjustedResolution.Y = FMath::RoundToInt(MaxTextureResolution / AspectRatio);
		}
		else if (AspectRatio > 0.0f)
		{
			// Taller than wide
			AdjustedResolution.Y = MaxTextureResolution;
			AdjustedResolution.X = FMath::RoundToInt(MaxTextureResolution * AspectRatio);
		}
	}

	// Generate the Charuco board texture using the subsystem abstraction
	if (UCameraCalibrationSubsystem* Subsystem = GEngine ? GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>() : nullptr)
	{
		CharucoBoardTexture = Subsystem->GenerateCharucoBoardTexture(Config, AdjustedResolution, 0);
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "CameraCalibrationSubsystem not available");
		CharucoBoardTexture = nullptr;
	}
	
	if (!CharucoBoardTexture)
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Failed to create Charuco texture");
		return;
	}

	// Apply texture to material if available
	if (CharucoBoardTexture && DynamicBoardMaterial)
	{
		DynamicBoardMaterial->SetTextureParameterValue(TEXT("Texture"), CharucoBoardTexture);
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Cannot apply texture - CharucoBoardTexture: %ls, DynamicBoardMaterial: %ls",
			CharucoBoardTexture ? TEXT("Valid") : TEXT("NULL"),
			DynamicBoardMaterial ? TEXT("Valid") : TEXT("NULL"));
	}
}

void ACameraCalibrationCharucoBoard::UpdateCalibrationPointPositions()
{
	// Position the calibration points at the inner corners.
	// Board origin is now at bottom-left outer corner, so inner corners are simply offset by SquareSideLength
	
	if (TopLeft)
	{
		TopLeft->SetRelativeLocation(FVector(0, SquareSideLength, (NumCornerRows) * SquareSideLength));
	}

	if (TopRight)
	{
		TopRight->SetRelativeLocation(FVector(0, NumCornerCols * SquareSideLength, (NumCornerRows) * SquareSideLength));
	}

	if (BottomLeft)
	{
		BottomLeft->SetRelativeLocation(FVector(0, SquareSideLength, SquareSideLength));
	}

	if (BottomRight)
	{
		BottomRight->SetRelativeLocation(FVector(0, NumCornerCols * SquareSideLength, SquareSideLength));
	}

	if (Center)
	{
		// Center of the inner corner grid
		float CenterY = (NumCornerCols + 1) * 0.5f * SquareSideLength;
		float CenterZ = (NumCornerRows + 1) * 0.5f * SquareSideLength;
		Center->SetRelativeLocation(FVector(0, CenterY, CenterZ));
	}
}

int32 ACameraCalibrationCharucoBoard::GetNumCorners() const
{
	return NumCornerCols * NumCornerRows;
}

FVector ACameraCalibrationCharucoBoard::GetCornerPosition(int32 CornerId) const
{
	if (CornerId < 0 || CornerId >= NumCornerCols * NumCornerRows)
	{
		return FVector::ZeroVector;
	}

	int32 Row = CornerId / NumCornerCols;
	int32 Col = CornerId % NumCornerCols;

	// Calculate local position using the board's coordinate system
	// Inner corners start at (1,1), that is, offset by one square from board origin
	FVector LocalPosition = SquareSideLength * FVector(0, Col + 1, Row + 1);

	// Transform to world coordinates
	return GetActorTransform().TransformPosition(LocalPosition);
}