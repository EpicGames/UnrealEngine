// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationCheckerboard.h"

#include "CalibrationPointComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraCalibrationCheckerboard)

ACameraCalibrationCheckerboard::ACameraCalibrationCheckerboard() : AActor()
{
	Root = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
	RootComponent = Root;
	RootComponent->SetVisibility(true);

	TopLeft     = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("TopLeft"));
	TopRight    = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("TopRight"));
	BottomLeft  = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("BottomLeft"));
	BottomRight = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("BottomRight"));
	Center      = CreateDefaultSubobject<UCalibrationPointComponent>(TEXT("Center"));

	TopLeft    ->SetupAttachment(RootComponent);
	TopRight   ->SetupAttachment(RootComponent);
	BottomLeft ->SetupAttachment(RootComponent);
	BottomRight->SetupAttachment(RootComponent);
	Center     ->SetupAttachment(RootComponent);

	if (!CubeMesh)
	{
		ConstructorHelpers::FObjectFinder<UStaticMesh> CubeFinder(TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'"), LOAD_Quiet | LOAD_NoWarn);

		if (CubeFinder.Succeeded())
		{
			CubeMesh = CubeFinder.Object;
		}
	}

	// Find default material for odd squares
	if (!OddCubeMaterial)
	{
		ConstructorHelpers::FObjectFinder<UMaterialInterface> MaterialFinder(TEXT("Material'/CameraCalibrationCore/Materials/M_DefaultOddSquare.M_DefaultOddSquare'"), LOAD_Quiet | LOAD_NoWarn);
		
		if (MaterialFinder.Succeeded())
		{
			OddCubeMaterial = MaterialFinder.Object;
		}
	}

	// Find default material for even squares
	if (!EvenCubeMaterial)
	{
		ConstructorHelpers::FObjectFinder<UMaterialInterface> MaterialFinder(TEXT("Material'/CameraCalibrationCore/Materials/M_DefaultEvenSquare.M_DefaultEvenSquare'"), LOAD_Quiet | LOAD_NoWarn);

		if (MaterialFinder.Succeeded())
		{
			EvenCubeMaterial = MaterialFinder.Object;
		}
	}

}

#if WITH_EDITOR
void ACameraCalibrationCheckerboard::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (   (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, NumCornerRows))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, NumCornerCols))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, SquareSideLength))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, Thickness))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, CubeMesh))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, OddCubeMaterial))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ACameraCalibrationCheckerboard, EvenCubeMaterial))
		)
	{
		Rebuild();
	}
}
#endif

void ACameraCalibrationCheckerboard::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
	Rebuild();
}

void ACameraCalibrationCheckerboard::ClearInstanceCheckeboardMeshComponents()
{
	// Need to cache because calling destroy will remove them from InstanceComponents
	TArray<UActorComponent*> CachedComponents = GetInstanceComponents();

	// Run in reverse to reduce memory churn when the components are removed from InstanceComponents
	for (UActorComponent* CachedComponent : ReverseIterate(CachedComponents))
	{
		if (UStaticMeshComponent* CachedMeshComponent = Cast<UStaticMeshComponent>(CachedComponent))
		{
			if (CachedMeshComponent->GetStaticMesh() == CubeMesh)
			{
				CachedMeshComponent->DestroyComponent();
			}
		}
	}
}

void ACameraCalibrationCheckerboard::Rebuild()
{
	Modify();

	// Removes the instance components that make up the checkerboard
	ClearInstanceCheckeboardMeshComponents();

	if (!CubeMesh)
	{
		return;
	}

	// Length of the static mesh we're using
	const float BasicCubeLength = 100.0f;

	// Origin of the static mesh we're using
	const FVector BasicCubeOrigin(50.0f, 50.0f, 50.0f);

	// Add cubes
	for (int32 RowIdx = 0; RowIdx < NumCornerRows+1; ++RowIdx)
	{
		for (int32 ColIdx = 0; ColIdx < NumCornerCols+1; ++ColIdx)
		{
			FTransform RelativeTransform = FTransform::Identity;

			RelativeTransform.SetScale3D(FVector(Thickness, SquareSideLength, SquareSideLength) / BasicCubeLength);
			RelativeTransform.SetLocation(FVector(Thickness, SquareSideLength * ColIdx, SquareSideLength * RowIdx) - BasicCubeOrigin * RelativeTransform.GetScale3D());

			UStaticMeshComponent* CubeComponent = NewObject<UStaticMeshComponent>(this);

			if (!CubeComponent)
			{
				break;
			}

			AddInstanceComponent(CubeComponent);

			CubeComponent->OnComponentCreated();
			CubeComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
			CubeComponent->SetRelativeTransform(RelativeTransform);
			CubeComponent->RegisterComponent();
			CubeComponent->SetStaticMesh(CubeMesh);

			bool bIsEven = !((RowIdx + ColIdx) % 2);

			if (bIsEven)
			{
				if (EvenCubeMaterial)
				{
					CubeComponent->SetMaterial(0, EvenCubeMaterial);
				}
			}
			else if (OddCubeMaterial)
			{
				CubeComponent->SetMaterial(0, OddCubeMaterial);
			}

			CubeComponent->SetVisibility(true);
		}
	}

	// Position the calibration points

	if (TopLeft)
	{
		TopLeft->SetRelativeLocation(SquareSideLength * FVector(0, 0, NumCornerRows-1));
	}

	if (TopRight)
	{
		TopRight->SetRelativeLocation(SquareSideLength * FVector(0, NumCornerCols-1, NumCornerRows-1));
	}

	if (BottomLeft)
	{
		BottomLeft->SetRelativeLocation(SquareSideLength * FVector(0, 0, 0));
	}

	if (BottomRight)
	{
		BottomRight->SetRelativeLocation(SquareSideLength * FVector(0, NumCornerCols-1, 0));
	}

	if (Center)
	{
		Center->SetRelativeLocation(0.5f * SquareSideLength * FVector(0, NumCornerCols-1, NumCornerRows-1));
	}
}
