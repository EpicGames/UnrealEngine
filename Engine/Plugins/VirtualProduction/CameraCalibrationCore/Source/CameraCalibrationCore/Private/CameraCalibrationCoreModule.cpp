// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationCoreModule.h"

#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationSubsystem.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "LensFile.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Models/AnamorphicLensModel.h"
#include "Models/BrownConradyDULensModel.h"
#include "Models/BrownConradyUDLensModel.h"
#include "Models/SphericalLensModel.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogCameraCalibrationCore);


static TAutoConsoleVariable<FString> CVarStartupLensFile(
	TEXT("CameraCalibration.StartupLensFile"),
	TEXT(""),
	TEXT("Startup Lens File\n"),
	ECVF_ReadOnly
);


void FCameraCalibrationCoreModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("CameraCalibrationCore"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/CameraCalibrationCore"), PluginShaderDir);

	ApplyStartupLensFile();
	RegisterDistortionModels();
}

void FCameraCalibrationCoreModule::ShutdownModule()
{
	UnregisterDistortionModels();

	if (PostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	}
}

void FCameraCalibrationCoreModule::ApplyStartupLensFile()
{
	auto ApplyLensFile = [this]()
	{
		ULensFile* StartupLensFile = nullptr;

		// Try to load from CVar
		{
			const FString LensFileName = CVarStartupLensFile.GetValueOnGameThread();

			if (LensFileName.Len())
			{
				if (UObject* Object = StaticLoadObject(ULensFile::StaticClass(), nullptr, *LensFileName))
				{
					StartupLensFile = CastChecked<ULensFile>(Object);
				}

				if (StartupLensFile)
				{
					UE_LOGF(LogCameraCalibrationCore, Display, "Loading Lens File specified in CVar CameraCalibration.StartupLensFile: '%ls'", *LensFileName);
				}
			}
		}

#if WITH_EDITOR
		// Try to load user settings
		if (StartupLensFile == nullptr)
		{
			StartupLensFile = GetDefault<UCameraCalibrationEditorSettings>()->GetUserLensFile();
			if (StartupLensFile)
			{
				UE_LOGF(LogCameraCalibrationCore, Display, "Loading Lens File specified in user settings: '%ls'", *StartupLensFile->GetName());
			}
		}
#endif

		// Try loading default lens file from project settings
		if (StartupLensFile == nullptr)
		{
			StartupLensFile = GetDefault<UCameraCalibrationSettings>()->GetStartupLensFile();

			if (StartupLensFile)
			{
				UE_LOGF(LogCameraCalibrationCore, Display, "Loading Lens File specified in project settings: '%ls'", *StartupLensFile->GetName());
			}
		}

		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		check(SubSystem);
		SubSystem->SetDefaultLensFile(StartupLensFile);
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			ApplyLensFile();
		}
		else
		{
			PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(ApplyLensFile);
		}
	}
}

void FCameraCalibrationCoreModule::RegisterDistortionModels()
{
	auto RegisterModels = [this]()
	{
		// Register all lens models defined in this module
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		SubSystem->RegisterDistortionModel(USphericalLensModel::StaticClass());
		SubSystem->RegisterDistortionModel(UAnamorphicLensModel::StaticClass());
		SubSystem->RegisterDistortionModel(UBrownConradyUDLensModel::StaticClass());
		SubSystem->RegisterDistortionModel(UBrownConradyDULensModel::StaticClass());
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterModels();
		}
		else
		{
			PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterModels);
		}
	}
}

void FCameraCalibrationCoreModule::UnregisterDistortionModels()
{
	if (GEngine)
	{
		// Unregister all lens models defined in this module
		if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
		{
			SubSystem->UnregisterDistortionModel(USphericalLensModel::StaticClass());
			SubSystem->UnregisterDistortionModel(UAnamorphicLensModel::StaticClass());
			SubSystem->UnregisterDistortionModel(UBrownConradyUDLensModel::StaticClass());
			SubSystem->UnregisterDistortionModel(UBrownConradyDULensModel::StaticClass());
		}
	}
}


IMPLEMENT_MODULE(FCameraCalibrationCoreModule, CameraCalibrationCore)