// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Delegates/IDelegateInstance.h"


class FCameraCalibrationCoreModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	void ApplyStartupLensFile();
	void RegisterDistortionModels();
	void UnregisterDistortionModels();

private:

	FDelegateHandle PostEngineInitHandle;
};
