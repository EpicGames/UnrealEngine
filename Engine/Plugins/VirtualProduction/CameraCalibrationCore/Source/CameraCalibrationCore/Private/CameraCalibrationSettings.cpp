// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationSettings.h"

#include "AnamorphicLensDistortionModelHandler.h"
#include "BrownConradyDULensDistortionModelHandler.h"
#include "BrownConradyUDLensDistortionModelHandler.h"
#include "CameraCalibrationSubsystem.h"
#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "SphericalLensDistortionModelHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraCalibrationSettings)

UCameraCalibrationSettings::UCameraCalibrationSettings()
{
	// Default materials to use for the spherical lens model
	DefaultUndistortionDisplacementMaterials.Add(USphericalLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_SphericalUndistortionDisplacementMap.M_SphericalUndistortionDisplacementMap"))));
	DefaultDistortionDisplacementMaterials.Add(USphericalLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_SphericalDistortionDisplacementMap.M_SphericalDistortionDisplacementMap"))));
	DefaultDistortionMaterials.Add(USphericalLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_DistortionPostProcess.M_DistortionPostProcess"))));

	// Default materials to use for the anamorphic lens model
	DefaultUndistortionDisplacementMaterials.Add(UAnamorphicLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_AnamorphicUndistortionDisplacementMap.M_AnamorphicUndistortionDisplacementMap"))));
	DefaultDistortionDisplacementMaterials.Add(UAnamorphicLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_AnamorphicDistortionDisplacementMap.M_AnamorphicDistortionDisplacementMap"))));
	DefaultDistortionMaterials.Add(UAnamorphicLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_DistortionPostProcess.M_DistortionPostProcess"))));

	// Default materials to use for the Brown-Conrady U-D lens model
	DefaultUndistortionDisplacementMaterials.Add(UBrownConradyUDLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_BrownConradyUDUndistortionDisplacementMap.M_BrownConradyUDUndistortionDisplacementMap"))));
	DefaultDistortionDisplacementMaterials.Add(UBrownConradyUDLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_BrownConradyUDDistortionDisplacementMap.M_BrownConradyUDDistortionDisplacementMap"))));
	DefaultDistortionMaterials.Add(UBrownConradyUDLensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_DistortionPostProcess.M_DistortionPostProcess"))));

	// Default materials to use for the Brown-Conrady D-U lens model
	DefaultUndistortionDisplacementMaterials.Add(UBrownConradyDULensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_BrownConradyDUUndistortionDisplacementMap.M_BrownConradyDUUndistortionDisplacementMap"))));
	DefaultDistortionDisplacementMaterials.Add(UBrownConradyDULensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_BrownConradyDUDistortionDisplacementMap.M_BrownConradyDUDistortionDisplacementMap"))));
	DefaultDistortionMaterials.Add(UBrownConradyDULensDistortionModelHandler::StaticClass(), TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(TEXT("/CameraCalibrationCore/Materials/M_DistortionPostProcess.M_DistortionPostProcess"))));

	// Defaults priority such that the SVE runs before OpenColorIO and CompositeCore.
	SceneViewExtensionPriority = 100 + 10 /** OPENCOLORIO_SCENE_VIEW_EXTENSION_PRIORITY + 10 */;

#if WITH_EDITORONLY_DATA
	auto InitOverlayOverrides = [this]()
	{
		if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
		{
			for (const FName& OverlayName : SubSystem->GetOverlayMaterialNames())
			{
				if (!CalibrationOverlayMaterialOverrides.Contains(OverlayName))
				{
					CalibrationOverlayMaterialOverrides.Add(OverlayName, nullptr);
				}
			}
		}
	};

	PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(InitOverlayOverrides);
#endif
}

FName UCameraCalibrationSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UCameraCalibrationSettings::GetSectionText() const
{
	return NSLOCTEXT("CameraCalibrationPlugin", "CameraCalibrationSettingsSection", "Camera Calibration");
}

FName UCameraCalibrationSettings::GetSectionName() const
{
	return TEXT("Camera Calibration");
}

#endif

ULensFile* UCameraCalibrationSettings::GetStartupLensFile() const
{
	return StartupLensFile.LoadSynchronous();
}

UMaterialInterface* UCameraCalibrationSettings::GetDefaultUndistortionDisplacementMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const
{
	const TSoftObjectPtr<UMaterialInterface>* UndistortionDisplacementMaterial = DefaultUndistortionDisplacementMaterials.Find(InModelHandler);
	if (!UndistortionDisplacementMaterial)
	{
		return nullptr;
	}
	return UndistortionDisplacementMaterial->LoadSynchronous();
}

UMaterialInterface* UCameraCalibrationSettings::GetDefaultDistortionDisplacementMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const
{
	const TSoftObjectPtr<UMaterialInterface>* DistortionDisplacementMaterial = DefaultDistortionDisplacementMaterials.Find(InModelHandler);
	if (!DistortionDisplacementMaterial)
	{
		return nullptr;
	}
	return DistortionDisplacementMaterial->LoadSynchronous();
}

UMaterialInterface* UCameraCalibrationSettings::GetDefaultDistortionMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const
{
	const TSoftObjectPtr<UMaterialInterface>* DistortionMaterial = DefaultDistortionMaterials.Find(InModelHandler);
	if (!DistortionMaterial)
	{
		return nullptr;
	}
	return DistortionMaterial->LoadSynchronous();
}

#if WITH_EDITOR
void UCameraCalibrationSettings::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	const FEditPropertyChain::TDoubleLinkedListNode* ActiveMemberNode = PropertyChangedEvent.PropertyChain.GetActiveMemberNode();
	const FProperty* MemberProperty = ActiveMemberNode ? ActiveMemberNode->GetValue() : nullptr;
	if (!MemberProperty || MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCameraCalibrationSettings, DisplacementMapResolution))
	{
		DisplacementMapResolutionChangedDelegate.Broadcast(DisplacementMapResolution);
	}
	if (!MemberProperty || MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCameraCalibrationSettings, CalibrationInputTolerance))
	{
		CalibrationInputToleranceChangedDelegate.Broadcast(CalibrationInputTolerance);
	}
}

TArray<FName> UCameraCalibrationSettings::GetCalibrationOverlayMaterialOverrideNames() const
{
	TArray<FName> OutKeys;
#if WITH_EDITORONLY_DATA
	CalibrationOverlayMaterialOverrides.GetKeys(OutKeys);
#endif
	return OutKeys;
}

UMaterialInterface* UCameraCalibrationSettings::GetCalibrationOverlayMaterialOverride(const FName OverlayName) const
{
#if WITH_EDITORONLY_DATA
	const TSoftObjectPtr<UMaterialInterface>* OverlayMaterial = CalibrationOverlayMaterialOverrides.Find(OverlayName);
	if (!OverlayMaterial)
	{
		return nullptr;
	}
	return OverlayMaterial->LoadSynchronous();
#else 
	return nullptr;
#endif
}

FOnDisplacementMapResolutionChanged& UCameraCalibrationSettings::OnDisplacementMapResolutionChanged()
{
	return DisplacementMapResolutionChangedDelegate;
}

FOnCalibrationInputToleranceChanged& UCameraCalibrationSettings::OnCalibrationInputToleranceChanged()
{
	return CalibrationInputToleranceChangedDelegate;
}
#endif

UCameraCalibrationEditorSettings::UCameraCalibrationEditorSettings()
{
}

FName UCameraCalibrationEditorSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UCameraCalibrationEditorSettings::GetSectionText() const
{
	return NSLOCTEXT("CameraCalibrationEditorPlugin", "CameraCalibrationEditorSettingsSection", "Camera Calibration Editor");
}

FName UCameraCalibrationEditorSettings::GetSectionName() const
{
	return TEXT("Camera Calibration Editor");
}
#endif

ULensFile* UCameraCalibrationEditorSettings::GetUserLensFile() const
{
#if WITH_EDITOR
	return UserLensFile.LoadSynchronous();
#else
	return nullptr;
#endif
}

void UCameraCalibrationEditorSettings::SetUserLensFile(ULensFile* InLensFile)
{
#if WITH_EDITOR
	UserLensFile = InLensFile;
	SaveConfig();
#endif
}
