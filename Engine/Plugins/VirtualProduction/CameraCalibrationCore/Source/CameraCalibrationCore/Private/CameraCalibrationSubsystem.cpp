// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCalibrationSubsystem.h"

#include "CameraCalibrationCharucoBoard.h"
#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "CineCameraComponent.h"
#include "Engine/TimecodeProvider.h"
#include "Engine/Texture2D.h"
#include "GameFramework/Actor.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(CameraCalibrationSubsystem)

#define LOCTEXT_NAMESPACE "CameraCalibrationSubsystem"



ULensFile* UCameraCalibrationSubsystem::GetDefaultLensFile() const
{
	return DefaultLensFile;
}

void UCameraCalibrationSubsystem::SetDefaultLensFile(ULensFile* NewDefaultLensFile)
{
	//Todo : Add callbacks when default lens file changes
	DefaultLensFile = NewDefaultLensFile;
}

ULensFile* UCameraCalibrationSubsystem::GetLensFile(const FLensFilePicker& Picker) const
{
	if (Picker.bUseDefaultLensFile)
	{
		return GetDefaultLensFile();
	}
	else
	{
		return Picker.LensFile;
	}
}

TArray<ULensDistortionModelHandlerBase*> UCameraCalibrationSubsystem::GetDistortionModelHandlers(UCineCameraComponent* Component)
{
	// This function has been deprecated. The implementation has been changed to provide some backwards compatibility, but code should be updated to not call this function.
	TArray<ULensDistortionModelHandlerBase*> Handlers;
	return Handlers;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ULensDistortionModelHandlerBase* UCameraCalibrationSubsystem::FindDistortionModelHandler(FDistortionHandlerPicker& DistortionHandlerPicker, bool bUpdatePicker) const
{
	// This function has been deprecated. The implementation has been changed to provide some backwards compatibility, but code should be updated to not call this function.
	TArray<ULensDistortionModelHandlerBase*> Handlers;
	return nullptr;
}

ULensDistortionModelHandlerBase* UCameraCalibrationSubsystem::FindOrCreateDistortionModelHandler(FDistortionHandlerPicker& DistortionHandlerPicker, const TSubclassOf<ULensModel> LensModelClass)
{
	// This function has been deprecated. The implementation has been changed to provide some backwards compatibility, but code should be updated to not call this function.
	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCameraCalibrationSubsystem::UnregisterDistortionModelHandler(UCineCameraComponent* Component, ULensDistortionModelHandlerBase* Handler)
{
	// This function has been deprecated.
}

void UCameraCalibrationSubsystem::RegisterDistortionModel(TSubclassOf<ULensModel> LensModel)
{
	// Since this may be looked by from a live link stream, we relax the lookup to be case-insensitive

	const FName ModelName = LensModel->GetDefaultObject<ULensModel>()->GetModelName();
	const FString ModelNameStr = ModelName.ToString().ToLower();

	LensModelMap.Add(FName(*ModelNameStr), LensModel);
}

void UCameraCalibrationSubsystem::UnregisterDistortionModel(TSubclassOf<ULensModel> LensModel)
{
	const FName ModelName = LensModel->GetDefaultObject<ULensModel>()->GetModelName();
	const FString ModelNameStr = ModelName.ToString().ToLower();

	LensModelMap.Remove(FName(*ModelNameStr));
}

void UCameraCalibrationSubsystem::RegisterOverlayMaterial(const FName& MaterialName, const FSoftObjectPath& MaterialPath)
{
	RegisteredOverlayMaterials.Add(MaterialName, TSoftObjectPtr<UMaterialInterface>(MaterialPath));
}

void UCameraCalibrationSubsystem::UnregisterOverlayMaterial(const FName& MaterialName)
{
	RegisteredOverlayMaterials.Remove(MaterialName);
}

TSubclassOf<ULensModel> UCameraCalibrationSubsystem::GetRegisteredLensModel(FName ModelName) const
{
	// Keys are stored in lower case to make the lookup case-insensitive.

	const FString ModelNameStr = ModelName.ToString().ToLower();
	const FName LowercaseModelName = FName(*ModelNameStr);
	
	if (LensModelMap.Contains(LowercaseModelName))
	{
		return LensModelMap[LowercaseModelName];
	}

	return nullptr;
}

TSubclassOf<UCameraNodalOffsetAlgo> UCameraCalibrationSubsystem::GetCameraNodalOffsetAlgo(FName Name) const
{
	if (CameraNodalOffsetAlgosMap.Contains(Name))
	{
		return CameraNodalOffsetAlgosMap[Name];
	}
	return nullptr;
}

TArray<FName> UCameraCalibrationSubsystem::GetCameraNodalOffsetAlgos() const
{
	TArray<FName> OutKeys;
	CameraNodalOffsetAlgosMap.GetKeys(OutKeys);
	return OutKeys;
}

TSubclassOf<UCameraImageCenterAlgo> UCameraCalibrationSubsystem::GetCameraImageCenterAlgo(FName Name) const
{
	if (CameraImageCenterAlgosMap.Contains(Name))
	{
		return CameraImageCenterAlgosMap[Name];
	}
	return nullptr;
}

TArray<FName> UCameraCalibrationSubsystem::GetCameraImageCenterAlgos() const
{
	TArray<FName> OutKeys;
	CameraImageCenterAlgosMap.GetKeys(OutKeys);
	return OutKeys;
}

UMaterialInterface* UCameraCalibrationSubsystem::GetOverlayMaterial(const FName& OverlayName) const
{
#if WITH_EDITOR
	if (UMaterialInterface* DefaultMaterial = GetDefault<UCameraCalibrationSettings>()->GetCalibrationOverlayMaterialOverride(OverlayName))
	{
		return DefaultMaterial;
	}
#endif

	const TSoftObjectPtr<UMaterialInterface> OverlayMaterial = RegisteredOverlayMaterials.FindRef(OverlayName);

	return OverlayMaterial.LoadSynchronous();
}

TArray<FName> UCameraCalibrationSubsystem::GetOverlayMaterialNames() const
{
	// Use a set to avoid duplicates when combining the registered overlays with the set of overrides
	TSet<FName> OverlayNames;

	TArray<FName> OutKeys;
	RegisteredOverlayMaterials.GetKeys(OutKeys);

	OverlayNames.Append(OutKeys);

#if WITH_EDITOR
	OverlayNames.Append(GetDefault<UCameraCalibrationSettings>()->GetCalibrationOverlayMaterialOverrideNames());
#endif

	return OverlayNames.Array();
}

TSubclassOf<UCameraCalibrationStep> UCameraCalibrationSubsystem::GetCameraCalibrationStep(FName Name) const
{
	if (CameraCalibrationStepsMap.Contains(Name))
	{
		return CameraCalibrationStepsMap[Name];
	}
	return nullptr;
}

TArray<FName> UCameraCalibrationSubsystem::GetCameraCalibrationSteps() const
{
	TArray<FName> OutKeys;
	CameraCalibrationStepsMap.GetKeys(OutKeys);
	return OutKeys;
}

void UCameraCalibrationSubsystem::SetLensDistortionSVEState(ACameraActor* CameraActor, FDisplacementMapBlendingParams DistortionState, ULensDistortionModelHandlerBase* LensDistortionHandler, EDistortionRenderingMode InRenderingMode)
{
	SceneViewExtension->UpdateDistortionState_AnyThread(CameraActor, DistortionState, LensDistortionHandler, InRenderingMode);
}

void UCameraCalibrationSubsystem::ClearLensDistortionSVEState(ACameraActor* CameraActor)
{
	SceneViewExtension->ClearDistortionState_AnyThread(CameraActor);
}

const TSharedPtr<FLensDistortionSceneViewExtension, ESPMode::ThreadSafe>& UCameraCalibrationSubsystem::GetSceneViewExtension() const
{
	return SceneViewExtension;
}

void UCameraCalibrationSubsystem::SetCharucoBoardTextureGenerator(FGenerateCharucoBoardDelegate InDelegate)
{
	CharucoBoardTextureGenerator = InDelegate;
}

UTexture2D* UCameraCalibrationSubsystem::GenerateCharucoBoardTexture(const FCharucoBoardConfigShadow& BoardConfig, FIntPoint ImageSize, int32 MarginSize)
{
	if (CharucoBoardTextureGenerator.IsBound())
	{
		return CharucoBoardTextureGenerator.Execute(BoardConfig, ImageSize, MarginSize);
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Charuco board texture generator not registered. OpenCV dependency may not be available.");
		return nullptr;
	}
}

void UCameraCalibrationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	SceneViewExtension = FSceneViewExtensions::NewExtension<FLensDistortionSceneViewExtension>();

	PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([&]()
	{
		// Find Image Center Algos
		{
			TArray<TSubclassOf<UCameraImageCenterAlgo>> Algos;

			for (TObjectIterator<UClass> AlgoIt; AlgoIt; ++AlgoIt)
			{
				if (AlgoIt->IsChildOf(UCameraImageCenterAlgo::StaticClass()) && !AlgoIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
				{
					const UCameraImageCenterAlgo* Algo = CastChecked<UCameraImageCenterAlgo>(AlgoIt->GetDefaultObject());
					CameraImageCenterAlgosMap.Add(Algo->FriendlyName(), TSubclassOf<UCameraImageCenterAlgo>(*AlgoIt));
				}
			}
		}

		// Find Nodal Offset Algos
		{
			TArray<TSubclassOf<UCameraNodalOffsetAlgo>> Algos;

			for (TObjectIterator<UClass> AlgoIt; AlgoIt; ++AlgoIt)
			{
				if (AlgoIt->IsChildOf(UCameraNodalOffsetAlgo::StaticClass()) && !AlgoIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
				{
					const UCameraNodalOffsetAlgo* Algo = CastChecked<UCameraNodalOffsetAlgo>(AlgoIt->GetDefaultObject());
					CameraNodalOffsetAlgosMap.Add(Algo->FriendlyName(), TSubclassOf<UCameraNodalOffsetAlgo>(*AlgoIt));
				}
			}
		}

		// Find Calibration Steps
		{
			TArray<TSubclassOf<UCameraCalibrationStep>> Steps;

			for (TObjectIterator<UClass> StepIt; StepIt; ++StepIt)
			{
				if (StepIt->IsChildOf(UCameraCalibrationStep::StaticClass()) && !StepIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
				{
					CameraCalibrationStepsMap.Add(StepIt->GetFName(), TSubclassOf<UCameraCalibrationStep>(*StepIt));
				}
			}
		}
	});
}

void UCameraCalibrationSubsystem::Deinitialize()
{
	SceneViewExtension.Reset();
	SceneViewExtension = nullptr;
	
	LensModelMap.Empty(0);
	CameraImageCenterAlgosMap.Empty(0);
	CameraNodalOffsetAlgosMap.Empty(0);
	CameraCalibrationStepsMap.Empty(0);

	if (PostEngineInitHandle.IsValid())
	{
		FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	}

	Super::Deinitialize();
}

#undef LOCTEXT_NAMESPACE


