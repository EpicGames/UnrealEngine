// Copyright Epic Games, Inc. All Rights Reserved.


#include "CameraCalibrationUtils.h"

#include "Math/UnrealMathUtility.h"

bool FCameraCalibrationUtils::IsNearlyEqual(const FTransform& A, const FTransform& B, float MaxLocationDelta, float MaxAngleDeltaDegrees)
{
	// Location check

	const float LocationDeltaInCm = (B.GetLocation() - A.GetLocation()).Size();

	if (LocationDeltaInCm > MaxLocationDelta)
	{
		return false;
	}

	// Rotation check

	const float AngularDistanceRadians = FMath::Abs(A.GetRotation().AngularDistance(B.GetRotation()));

	if (AngularDistanceRadians > FMath::DegreesToRadians(MaxAngleDeltaDegrees))
	{
		return false;
	}

	return true;
}

