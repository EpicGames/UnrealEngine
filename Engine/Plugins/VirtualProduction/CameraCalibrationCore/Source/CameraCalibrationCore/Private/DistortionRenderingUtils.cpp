// Copyright Epic Games, Inc. All Rights Reserved.


#include "DistortionRenderingUtils.h"

#include "CameraCalibrationCoreLog.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIUtilities.h"
#include "RHIResourceUtils.h"

class FUndistortImagePointsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FUndistortImagePointsCS);

	SHADER_USE_PARAMETER_STRUCT(FUndistortImagePointsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, DistortionMap)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortionMapSampler)
		SHADER_PARAMETER_SRV(StructuredBuffer<FVector2f>, InputPoints)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<FVector2f>, UndistortedPoints)
	END_SHADER_PARAMETER_STRUCT()

public:
	// Called by the engine to determine which permutations to compile for this shader
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

// --------------------------------------------------------------------------------------------------------------------
IMPLEMENT_GLOBAL_SHADER(FUndistortImagePointsCS, "/Plugin/CameraCalibrationCore/Private/UndistortImagePoints.usf", "UndistortImagePointsCS", SF_Compute);

namespace DistortionRenderingUtils
{
	void UndistortImagePoints(UTextureRenderTarget2D* DistortionMap, TArray<FVector2D> ImagePoints, TArray<FVector2D>& OutUndistortedPoints)
	{
		// Convert the input points from doubles to floats
		TArray<FVector2f> ImagePointsFloat;
		ImagePointsFloat.Reserve(ImagePoints.Num());
		for (FVector2D Point : ImagePoints)
		{
			ImagePointsFloat.Add(FVector2f(Point.X, Point.Y));
		}

		TArray<FVector2f> UndistortedPointsFloat;
		UndistortImagePoints(DistortionMap, ImagePointsFloat, UndistortedPointsFloat);

		// Convert the output points from floats to doubles
		OutUndistortedPoints.Reserve(UndistortedPointsFloat.Num());
		for (FVector2f Point : UndistortedPointsFloat)
		{
			OutUndistortedPoints.Add(FVector2D(Point.X, Point.Y));
		}
	}

	void UndistortImagePoints(UTextureRenderTarget2D* DistortionMap, TArray<FVector2f> ImagePoints, TArray<FVector2f>& OutUndistortedPoints)
	{
		if ((DistortionMap == nullptr) || (ImagePoints.Num() < 1))
		{
			return;
		}

		const int32 NumPoints = ImagePoints.Num();
		const FTextureRenderTargetResource* const DistortionMapResource = DistortionMap->GameThread_GetRenderTargetResource();
		
		// Ensure the output array has the correct size
		OutUndistortedPoints.SetNum(NumPoints);

		ENQUEUE_RENDER_COMMAND(DistortionRenderingUtils_UndistortImagePoints)(
			[DistortionMapResource, NumPoints, Points = MoveTemp(ImagePoints), &OutUndistortedPoints](FRHICommandListImmediate& RHICmdList)
			{
				// Set up parameters
				FUndistortImagePointsCS::FParameters Parameters;
				Parameters.DistortionMap = DistortionMapResource->TextureRHI;
				Parameters.DistortionMapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

				TResourceArray<FVector2f> EmptyBuffer;
				EmptyBuffer.AddZeroed(NumPoints);

				const uint32 BufferSize = sizeof(FVector2f) * NumPoints;

				// Create an SRV for the input buffer of image points
				FBufferRHIRef InputPointsBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
					RHICmdList,
					TEXT("ImagePointsInitialData"),
					EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource,
					MakeConstArrayView(Points)
				);

				FShaderResourceViewRHIRef InputPointsSRV = RHICmdList.CreateShaderResourceView(InputPointsBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(InputPointsBuffer));
				Parameters.InputPoints = InputPointsSRV;

				// Create a RWBuffer to use as a UAV for the output buffer of undistorted points
				FRWBuffer UndistortedPointsBuffer;
				UndistortedPointsBuffer.Initialize(RHICmdList, TEXT("UndistortedPointsBuffer"), sizeof(FVector2f), NumPoints, PF_G32R32F, ERHIAccess::UAVCompute, BUF_SourceCopy | BUF_UnorderedAccess, &EmptyBuffer);
				Parameters.UndistortedPoints = UndistortedPointsBuffer.UAV;

				// Dispatch compute shader
				TShaderMapRef<FUndistortImagePointsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumPoints, 1, 1));

				// Copy the undistorted points buffer to a staging buffer to read it back on the CPU
				RHICmdList.Transition(FRHITransitionInfo(UndistortedPointsBuffer.UAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
				FStagingBufferRHIRef DestinationStagingBuffer = RHICreateStagingBuffer();
				RHICmdList.CopyToStagingBuffer(UndistortedPointsBuffer.Buffer, DestinationStagingBuffer, 0, BufferSize);

				// Wait to ensure that the staging buffer is ready to read
				RHICmdList.SubmitAndBlockUntilGPUIdle();

				// Copy data out of the staging buffer
				FVector2f* UndistortedPointData = static_cast<FVector2f*>(DestinationStagingBuffer->Lock(0, BufferSize));

				if (UndistortedPointData)
				{
					for (int32 Index = 0; Index < NumPoints; ++Index)
					{
						OutUndistortedPoints[Index] = UndistortedPointData[Index];
					}
				}
				else
				{
					UE_LOGF(LogCameraCalibrationCore, Warning, "DistortionRenderingUtils::UndistortImagePoints: Failed to read undistorted points from GPU, so filling with zeroes.");

					for (int32 Index = 0; Index < NumPoints; ++Index)
					{
						OutUndistortedPoints[Index] = FVector2f::ZeroVector;
					}
				}

				DestinationStagingBuffer->Unlock();
			});

		// Ensure that all rendering commands have been issued so that OutUndistortedPoints() is valid and has correct data 
		FlushRenderingCommands();
	}
}
