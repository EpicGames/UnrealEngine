// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayCameras/LensCalibrationCameraNode.h"

#include "Core/CameraEvaluationContext.h"
#include "LensData.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensFile.h"
#include "Models/LensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensCalibrationCameraNode)

namespace UE::Cameras
{

class FLensCalibrationCameraNodeEvaluator : public FCameraNodeEvaluator
{
	UE_DECLARE_CAMERA_NODE_EVALUATOR(CAMERACALIBRATIONCORE_API, FLensCalibrationCameraNodeEvaluator)

protected:

	// FCameraNodeEvaluator interface.
	virtual void OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult) override;
	virtual void OnAddReferencedObjects(FReferenceCollector& Collector) override;

private:

	TObjectPtr<const ULensFile> LensFile;
	TObjectPtr<ULensDistortionModelHandlerBase> DistortionHandler;
};

UE_DEFINE_CAMERA_NODE_EVALUATOR(FLensCalibrationCameraNodeEvaluator)

void FLensCalibrationCameraNodeEvaluator::OnInitialize(const FCameraNodeEvaluatorInitializeParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	SetNodeEvaluatorFlags(ECameraNodeEvaluatorFlags::None);

	const ULensCalibrationCameraNode* LensCalibrationNode = GetCameraNodeAs<ULensCalibrationCameraNode>();
	if (LensCalibrationNode->LensFile)
	{
		LensFile = LensCalibrationNode->LensFile;

		if (TSubclassOf<ULensDistortionModelHandlerBase> DistortionHandlerClass = ULensModel::GetHandlerClass(LensFile->LensInfo.LensModel))
		{
			UObject* OuterObject = Params.EvaluationContext->GetOwner();
			if (!OuterObject)
			{
				OuterObject = GetTransientPackage();
			}

			DistortionHandler = NewObject<ULensDistortionModelHandlerBase>(OuterObject, DistortionHandlerClass, NAME_None, RF_Transient);
		}
	}
}

void FLensCalibrationCameraNodeEvaluator::OnRun(const FCameraNodeEvaluationParams& Params, FCameraNodeEvaluationResult& OutResult)
{
	FCameraPose& CameraPose = OutResult.CameraPose;

	FVector2d SensorDimensions(CameraPose.GetSensorWidth(), CameraPose.GetSensorHeight());
	if (SensorDimensions.X <= 0.f || SensorDimensions.Y <= 0.f)
	{
		// We don't have a sensor size, let's use the standard size for our lens definition.
		SensorDimensions = LensFile->LensInfo.SensorDimensions;
	}

	float FocalLength = CameraPose.GetFocalLength();
	if (FocalLength <= 0.f)
	{
		// We don't have a valid focal length, so we have to compute it from the FOV. This isn't super
		// good since we're obviously trying to do physical lens modeling with a setup that isn't made
		// for it but hey, let's try.
		ensure(CameraPose.GetFieldOfView() > 0.f);

		FocalLength = 0.5f * SensorDimensions.X / FMath::Tan(FMath::DegreesToRadians(0.5f * CameraPose.GetFieldOfView()));
	}

	// Compute focal length interpolation.
	if (LensFile)
	{
		FFocalLengthInfo FocalLengthInfo;
		if (LensFile->EvaluateFocalLength(CameraPose.GetFocusDistance(), FocalLength, FocalLengthInfo))
		{
			// FocalLengthInfo has normalized values that need to be denormalized using the sensor size to get millimeters.
			const float InterpolatedFocalLength = FocalLengthInfo.FxFy.X * SensorDimensions.X;
			if (InterpolatedFocalLength > 0.f)
			{
				CameraPose.SetFocalLength(InterpolatedFocalLength);
			}
		}
	}

	// Compute lens distortion.
	if (DistortionHandler)
	{
		if (LensFile->EvaluateDistortionData(CameraPose.GetFocusDistance(), FocalLength, SensorDimensions, DistortionHandler))
		{
			FPostProcessSettings& PostProcessSettings = OutResult.PostProcessSettings.Get();
			PostProcessSettings.AddBlendable(DistortionHandler->GetDistortionMID(), 1.0f);

			// Distortion overscan is in percentages (defaults to 100%), but FCameraPose stores overscan in additive
			// percentages (defaults to +0%), hence the -1 here.
			const float DistortionOverscan = FMath::Clamp(DistortionHandler->GetOverscanFactor() - 1.0f, 0.0f, 1.0f);

			// We may need option to accumulate overscan with other sources instead of overwriting it.
			CameraPose.SetOverscan(DistortionOverscan);
		}
	}
}

void FLensCalibrationCameraNodeEvaluator::OnAddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(LensFile);
	Collector.AddReferencedObject(DistortionHandler);
}

}  // namespace UE::Cameras

FCameraNodeEvaluatorPtr ULensCalibrationCameraNode::OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const
{
	using namespace UE::Cameras;
	return Builder.BuildEvaluator<FLensCalibrationCameraNodeEvaluator>();
}

