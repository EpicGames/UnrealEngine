// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionModelHandlerBase.h"

#include "Algo/MaxElement.h"
#include "CameraCalibrationSettings.h"
#include "Engine/World.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "MaterialShared.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensDistortionModelHandlerBase)

bool FLensDistortionState::operator==(const FLensDistortionState& Other) const
{
	return ((DistortionInfo.Parameters == Other.DistortionInfo.Parameters)
		&& (FocalLengthInfo.FxFy == Other.FocalLengthInfo.FxFy)
		&& (ImageCenter.PrincipalPoint == Other.ImageCenter.PrincipalPoint));
}

ULensDistortionModelHandlerBase::ULensDistortionModelHandlerBase()
{
	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		GetMutableDefault<UCameraCalibrationSettings>()->OnDisplacementMapResolutionChanged().AddUObject(this, &ULensDistortionModelHandlerBase::CreateDisplacementMaps);
#endif // WITH_EDITOR
	}
}

EDistortionRenderingMode ULensDistortionModelHandlerBase::GetPreferredRenderingMode() const
{
	// By default, reverse distorting models prefer the plugin pass for accuracy, forward distorting models prefer TSR
	return IsForwardDistorting() ? EDistortionRenderingMode::TemporalSuperResolution : EDistortionRenderingMode::PassAfterTonemap;
}

bool ULensDistortionModelHandlerBase::IsModelSupported(const TSubclassOf<ULensModel>& ModelToSupport) const
{
	return (LensModelClass == ModelToSupport);
}

void ULensDistortionModelHandlerBase::SetDistortionState(const FLensDistortionState& InNewState)
{
	// If the new state is equivalent to the current state, there is nothing to update. 
	if (CurrentState != InNewState)
	{
		CurrentState = InNewState;
		InterpretDistortionParameters();

		bIsDirty = true;
	}	
}

void ULensDistortionModelHandlerBase::SetCameraFilmback(const FCameraFilmbackSettings& InCameraFilmback)
{
	if (CameraFilmback != InCameraFilmback)
	{
		CameraFilmback = InCameraFilmback;
		InterpretDistortionParameters();

		bIsDirty = true;
	}
}

void ULensDistortionModelHandlerBase::PostInitProperties()
{
	Super::PostInitProperties();

	// Perform handler initialization, only on derived classes
	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		InitializeHandler();
	}

	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		if (LensModelClass)
		{
			LensModelClass->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(CurrentState.DistortionInfo.Parameters);
		}

		const FIntPoint DisplacementMapResolution = GetDefault<UCameraCalibrationSettings>()->GetDisplacementMapResolution();
		CreateDisplacementMaps(DisplacementMapResolution);
	}
}

void ULensDistortionModelHandlerBase::CreateDisplacementMaps(const FIntPoint DisplacementMapResolution)
{
	//Helper function to set material parameters of an MID
	const auto SetupDisplacementMap = [this, DisplacementMapResolution](UTextureRenderTarget2D* const DisplacementMap)
	{
		DisplacementMap->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RG16f;
		DisplacementMap->ClearColor = FLinearColor::Black;
		DisplacementMap->Filter = TF_Bilinear;
		DisplacementMap->AddressX = TA_Clamp;
		DisplacementMap->AddressY = TA_Clamp;
		DisplacementMap->bAutoGenerateMips = false;
		DisplacementMap->InitAutoFormat(DisplacementMapResolution.X, DisplacementMapResolution.Y);
		DisplacementMap->UpdateResourceImmediate(true);
	};

	// Create an undistortion displacement map
	UndistortionDisplacementMapRT = NewObject<UTextureRenderTarget2D>(this, MakeUniqueObjectName(this, UTextureRenderTarget2D::StaticClass(), TEXT("UndistortionDisplacementMapRT")));
	SetupDisplacementMap(UndistortionDisplacementMapRT);

	// Create a distortion displacement map
	DistortionDisplacementMapRT = NewObject<UTextureRenderTarget2D>(this, MakeUniqueObjectName(this, UTextureRenderTarget2D::StaticClass(), TEXT("DistortionDisplacementMapRT")));
	SetupDisplacementMap(DistortionDisplacementMapRT);

	if (DistortionPostProcessMID)
	{
		DistortionPostProcessMID->SetTextureParameterValue("UndistortionDisplacementMap", UndistortionDisplacementMapRT);
		DistortionPostProcessMID->SetTextureParameterValue("DistortionDisplacementMap", DistortionDisplacementMapRT);
	}
}

#if WITH_EDITOR
void ULensDistortionModelHandlerBase::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	const FName MemberPropertyName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULensDistortionModelHandlerBase, CurrentState))
	{
		// Will need to revisit this init logic once we move to arbitrary lens model support 
		if (!DistortionPostProcessMID || !UndistortionDisplacementMapMID || !DistortionDisplacementMapMID)
		{
			InitDistortionMaterials();
		}

		bIsDirty = true;
		InterpretDistortionParameters();
		SetOverscanFactor(ComputeOverscanFactor());
		ProcessCurrentDistortion();
	}
}
#endif	

void ULensDistortionModelHandlerBase::SetOverscanFactor(float InOverscanFactor)
{
	if (!DistortionPostProcessMID || !UndistortionDisplacementMapMID || !DistortionDisplacementMapMID)
	{
		InitDistortionMaterials();
	}
	
	OverscanFactor = InOverscanFactor;
	if (DistortionPostProcessMID)
	{
		DistortionPostProcessMID->SetScalarParameterValue("overscan_factor", OverscanFactor);
	}
}

float ULensDistortionModelHandlerBase::ComputeOverscanFactor() const
{
	/* Distorted UV position in the view space:
					   ^ View space's Y
					   |
			  0        1        2

			  7                 3 --> View space's X

			  6        5        4
	*/
	const TArray<FVector2D> DistortedUVs =
	{
		FVector2D(0.0f, 0.0f),
		FVector2D(0.5f, 0.0f),
		FVector2D(1.0f, 0.0f),
		FVector2D(1.0f, 0.5f),
		FVector2D(1.0f, 1.0f),
		FVector2D(0.5f, 1.0f),
		FVector2D(0.0f, 1.0f),
		FVector2D(0.0f, 0.5f)
	};

	TArray<float> OverscanFactors;
	OverscanFactors.Reserve(DistortedUVs.Num());
	for (const FVector2D& DistortedUV : DistortedUVs)
	{
		const FVector2D UndistortedUV = ComputeUndistortedUV(DistortedUV);
		const float OverscanX = (DistortedUV.X != 0.5f) ? (UndistortedUV.X - 0.5f) / (DistortedUV.X - 0.5f) : 1.0f;
		const float OverscanY = (DistortedUV.Y != 0.5f) ? (UndistortedUV.Y - 0.5f) / (DistortedUV.Y - 0.5f) : 1.0f;
		OverscanFactors.Add(FMath::Max(OverscanX, OverscanY));
	}

	float NewOverscan = 1.0f;
	float* MaxOverscanFactor = Algo::MaxElement(OverscanFactors);
	if (MaxOverscanFactor != nullptr)
	{
		NewOverscan = FMath::Max(*MaxOverscanFactor, 1.0f);
	}

	return NewOverscan;
}

float ULensDistortionModelHandlerBase::ComputeInverseOverscanFactor() const
{
	/* Undistorted UV position in the view space:
					   ^ View space's Y
					   |
			  0        1        2

			  7                 3 --> View space's X

			  6        5        4
	*/
	const TArray<FVector2D> UndistortedUVs =
	{
		FVector2D(0.0f, 0.0f),
		FVector2D(0.5f, 0.0f),
		FVector2D(1.0f, 0.0f),
		FVector2D(1.0f, 0.5f),
		FVector2D(1.0f, 1.0f),
		FVector2D(0.5f, 1.0f),
		FVector2D(0.0f, 1.0f),
		FVector2D(0.0f, 0.5f)
	};

	TArray<float> OverscanFactors;
	OverscanFactors.Reserve(UndistortedUVs.Num());
	for (const FVector2D& UndistortedUV : UndistortedUVs)
	{
		const FVector2D DistortedUV = ComputeDistortedUV(UndistortedUV);
		const float OverscanX = (UndistortedUV.X != 0.5f) ? (DistortedUV.X - 0.5f) / (UndistortedUV.X - 0.5f) : 1.0f;
		const float OverscanY = (UndistortedUV.Y != 0.5f) ? (DistortedUV.Y - 0.5f) / (UndistortedUV.Y - 0.5f) : 1.0f;
		OverscanFactors.Add(FMath::Max(OverscanX, OverscanY));
	}

	float NewOverscan = 1.0f;
	float* MaxOverscanFactor = Algo::MaxElement(OverscanFactors);
	if (MaxOverscanFactor != nullptr)
	{
		NewOverscan = FMath::Max(*MaxOverscanFactor, 1.0f);
	}

	return NewOverscan;
}

TArray<FVector2D> ULensDistortionModelHandlerBase::GetDistortedUVs(TConstArrayView<FVector2D> UndistortedUVs) const
{
	TArray<FVector2D> DistortedUVs;
	DistortedUVs.Reserve(UndistortedUVs.Num());
	for (const FVector2D& UndistortedUV : UndistortedUVs)
	{
		const FVector2D DistortedUV = ComputeDistortedUV(UndistortedUV);
		DistortedUVs.Add(DistortedUV);
	}
	return DistortedUVs;
}

bool ULensDistortionModelHandlerBase::DrawUndistortionDisplacementMap(UTextureRenderTarget2D* DestinationTexture)
{
	if(DestinationTexture == nullptr)
	{
		return false;
	}
	
	if (!DistortionPostProcessMID || !UndistortionDisplacementMapMID)
	{
		InitDistortionMaterials();
	}
	
	UpdateMaterialParameters();

	// Draw the updated displacement map render target 
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, DestinationTexture, UndistortionDisplacementMapMID);

	return true;
}

bool ULensDistortionModelHandlerBase::DrawDistortionDisplacementMap(UTextureRenderTarget2D* DestinationTexture)
{
	if (DestinationTexture == nullptr)
	{
		return false;
	}

	if (!DistortionPostProcessMID || !DistortionDisplacementMapMID)
	{
		InitDistortionMaterials();
	}

	UpdateMaterialParameters();

	// Draw the updated displacement map render target 
	UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, DestinationTexture, DistortionDisplacementMapMID);

	return true;
}

bool ULensDistortionModelHandlerBase::IsDisplacementMapMaterialReady(UMaterialInstanceDynamic* MID)
{
	EShaderPlatform ShaderPlatform = GetWorld() ? GetFeatureLevelShaderPlatform_Checked(GetWorld()->GetFeatureLevel()) : GMaxRHIShaderPlatform;

	if (FMaterialResource* MaterialResource = MID->GetMaterialResource(ShaderPlatform))
	{
		if (MaterialResource->IsGameThreadShaderMapComplete())
		{
			return true;
		}

#if WITH_EDITOR
		MaterialResource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::ForceLocal);
#endif

		return MaterialResource->IsGameThreadShaderMapComplete();
	}

	return false;
}

void ULensDistortionModelHandlerBase::ProcessCurrentDistortion()
{
	bool bAreMaterialsReady = true;
	if (!UndistortionDisplacementMapMID || !DistortionDisplacementMapMID)
	{
		InitDistortionMaterials();
	}

	if (UndistortionDisplacementMapMID && !IsDisplacementMapMaterialReady(UndistortionDisplacementMapMID))
	{
		bAreMaterialsReady = false;
	}

	if (DistortionDisplacementMapMID && !IsDisplacementMapMaterialReady(DistortionDisplacementMapMID))
	{
		bAreMaterialsReady = false;
	}

	if (!bAreMaterialsReady)
	{
		return;
	}

	if(bIsDirty)
	{
		bIsDirty = false;

		InterpretDistortionParameters();

		UpdateMaterialParameters();

		// Draw the undistortion displacement map
		if (UndistortionDisplacementMapMID)
		{
			UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, UndistortionDisplacementMapRT, UndistortionDisplacementMapMID);
		}

		// Draw the distortion displacement map
		if (DistortionDisplacementMapMID)
		{
			UKismetRenderingLibrary::DrawMaterialToRenderTarget(this, DistortionDisplacementMapRT, DistortionDisplacementMapMID);
		}
	}
}
