// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionSceneViewExtension.h"

#include "CineCameraComponent.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "CameraCalibrationSettings.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "RHIResourceUtils.h"
#include "ScreenPass.h"
#include "ShaderParameterStruct.h"
#include "SystemTextures.h"
#include "TextureResource.h"
#include "SceneView.h"
#include "RenderGraph.h"
#include "RHI.h"

TAutoConsoleVariable<int32> CVarLensDistortionInvertGridDensity(
	TEXT("r.LensDistortion.InvertGridDensity"),
	96,
	TEXT("The number of squares drawn by the shader that inverts the distortion displacement map\n")
	TEXT("Value is clamped between 64 and 255.\n"),
	ECVF_RenderThreadSafe);

FLensDistortionSceneViewExtension::FLensDistortionSceneViewExtension(const FAutoRegister& AutoRegister)
	: FSceneViewExtensionBase(AutoRegister)
{
}

void FLensDistortionSceneViewExtension::UpdateDistortionState_AnyThread(ACameraActor* CameraActor, FDisplacementMapBlendingParams DistortionState, ULensDistortionModelHandlerBase* LensDistortionHandler, EDistortionRenderingMode InRenderingMode)
{
	FScopeLock ScopeLock(&DistortionStateMapCriticalSection);

	FCameraDistortionProxy CamProxy;
	CamProxy.Params = DistortionState;
	CamProxy.LensDistortionHandler = LensDistortionHandler;
	CamProxy.RenderingMode = InRenderingMode;

	if (UCameraComponent* Component = CameraActor->GetCameraComponent())
	{
		CamProxy.CameraOverscan = Component->Overscan + 1.0f;
		
		if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(Component))
		{
			CamProxy.FilmbackSettings = CineCameraComponent->Filmback;
		}
	}
	
	DistortionStateMap.Add(CameraActor->GetUniqueID(), MoveTemp(CamProxy));
}

bool FLensDistortionSceneViewExtension::HasDistortionState_AnyThread() const
{
	FScopeLock ScopeLock(&DistortionStateMapCriticalSection);
	return DistortionStateMap.Num() > 0;
}

TOptional<EDistortionRenderingMode> FLensDistortionSceneViewExtension::GetRenderingMode_AnyThread(uint32 InViewActorID) const
{
	FScopeLock ScopeLock(&DistortionStateMapCriticalSection);
	if (const FCameraDistortionProxy* Proxy = DistortionStateMap.Find(InViewActorID))
	{
		return Proxy->RenderingMode;
	}

	return {};
}

void FLensDistortionSceneViewExtension::ClearDistortionState_AnyThread(ACameraActor* CameraActor)
{
	FScopeLock ScopeLock(&DistortionStateMapCriticalSection);
	DistortionStateMap.Remove(CameraActor->GetUniqueID());
}


class FBlendDistortionDisplacementMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBlendDistortionDisplacementMapCS);
	SHADER_USE_PARAMETER_STRUCT(FBlendDistortionDisplacementMapCS, FGlobalShader);

	class FBlendType : SHADER_PERMUTATION_INT("BLEND_TYPE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FBlendType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, ThreadIdToUV)
		SHADER_PARAMETER(FVector2f, FxFyScale)
		SHADER_PARAMETER_ARRAY(FVector4f, PatchCorners, [4])
		SHADER_PARAMETER(float, EvalFocus)
		SHADER_PARAMETER(float, EvalZoom)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputDistortionMap1)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputDistortionMap2)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputDistortionMap3)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputDistortionMap4)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OverscanDistortionMap)
	END_SHADER_PARAMETER_STRUCT()

	// Called by the engine to determine which permutations to compile for this shader
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

IMPLEMENT_GLOBAL_SHADER(FBlendDistortionDisplacementMapCS, "/Plugin/CameraCalibrationCore/Private/BlendDisplacementMaps.usf", "MainCS", SF_Compute);

class FCropDistortionDisplacementMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCropDistortionDisplacementMapCS);
	SHADER_USE_PARAMETER_STRUCT(FCropDistortionDisplacementMapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InDistortionMapWithOverscan)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDistortionMap)
		SHADER_PARAMETER(FIntPoint, OverscanOffset)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCropDistortionDisplacementMapCS, "/Plugin/CameraCalibrationCore/Private/CropDisplacementMap.usf", "MainCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FInvertDisplacementParameters, )
	SHADER_PARAMETER(FIntPoint, GridDimensions)
	SHADER_PARAMETER(FVector2f, PixelToUV)
	SHADER_PARAMETER(FVector2f, PixelToOverscanUV)
	SHADER_PARAMETER(float, OverscanFactor)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DistortionMap)
	SHADER_PARAMETER_SAMPLER(SamplerState, DistortionMapSampler)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FInvertDisplacementVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FInvertDisplacementVS);
	SHADER_USE_PARAMETER_STRUCT(FInvertDisplacementVS, FGlobalShader);
	using FParameters = FInvertDisplacementParameters;
};

class FInvertDisplacementPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FInvertDisplacementPS);
	SHADER_USE_PARAMETER_STRUCT(FInvertDisplacementPS, FGlobalShader);
	using FParameters = FInvertDisplacementParameters;
};

IMPLEMENT_GLOBAL_SHADER(FInvertDisplacementVS, "/Plugin/CameraCalibrationCore/Private/InvertDisplacementMap.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FInvertDisplacementPS, "/Plugin/CameraCalibrationCore/Private/InvertDisplacementMap.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FFillSTDisplacementMapParameters, )
	SHADER_PARAMETER(FIntPoint, GridDimensions)
	SHADER_PARAMETER(FVector2f, PixelToUV)
	SHADER_PARAMETER(FVector2f, PixelToOverscannedUV)
	SHADER_PARAMETER(float, Overscan)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, UndisplacementMap)
	SHADER_PARAMETER_SAMPLER(SamplerState, UndisplacementMapSampler)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FFillSTDisplacementMapVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFillSTDisplacementMapVS);
	SHADER_USE_PARAMETER_STRUCT(FFillSTDisplacementMapVS, FGlobalShader);
	using FParameters = FFillSTDisplacementMapParameters;
};

class FFillSTDisplacementMapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFillSTDisplacementMapPS);
	SHADER_USE_PARAMETER_STRUCT(FFillSTDisplacementMapPS, FGlobalShader);
	using FParameters = FFillSTDisplacementMapParameters;
};

IMPLEMENT_GLOBAL_SHADER(FFillSTDisplacementMapVS, "/Plugin/CameraCalibrationCore/Private/FillSTDisplacementMap.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FFillSTDisplacementMapPS, "/Plugin/CameraCalibrationCore/Private/FillSTDisplacementMap.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FRecenterSTDisplacementMapParameters, )
	SHADER_PARAMETER(float, Overscan)
	SHADER_PARAMETER(FVector2f, STMapInvSize)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, STMap)
	SHADER_PARAMETER_SAMPLER(SamplerState, STMapSampler)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FRecenterSTDisplacementMapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FRecenterSTDisplacementMapPS);
	SHADER_USE_PARAMETER_STRUCT(FRecenterSTDisplacementMapPS, FGlobalShader);
	using FParameters = FRecenterSTDisplacementMapParameters;
};

IMPLEMENT_GLOBAL_SHADER(FRecenterSTDisplacementMapPS, "/Plugin/CameraCalibrationCore/Private/RecenterSTDisplacementMaps.usf", "MainPS", SF_Pixel);

void FLensDistortionSceneViewExtension::DrawDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FLensDistortionState& CurrentState, ULensDistortionModelHandlerBase* ModelHandler, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMapWithOverscan) const
{
	if (CurrentState.DistortionInfo.Parameters.IsEmpty())
	{
		OutDistortionMapWithOverscan = GSystemTextures.GetBlackDummy(GraphBuilder);
		return;
	}

	// Check if we have a valid handler
	if (!ModelHandler)
	{
		UE_LOGF(LogTemp, Warning, "No model handler available in DrawDisplacementMap_RenderThread, using dummy texture");
		OutDistortionMapWithOverscan = GSystemTextures.GetBlackDummy(GraphBuilder);
		return;
	}

	FIntPoint DistortionMapResolution = OutDistortionMapWithOverscan->Desc.Extent;

	// Fully generic shader execution through handler - enables unlimited third-party extensibility
	ModelHandler->ExecuteDistortionShader(GraphBuilder, CurrentState, InverseOverscan, CameraOverscan, SensorSize, OutDistortionMapWithOverscan);
}

void FLensDistortionSceneViewExtension::CropDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDistortionMapWithOverscan, FRDGTextureRef& OutDistortionMap) const
{
	FCropDistortionDisplacementMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCropDistortionDisplacementMapCS::FParameters>();

	PassParameters->InDistortionMapWithOverscan = GraphBuilder.CreateSRV(InDistortionMapWithOverscan);
	PassParameters->OutDistortionMap = GraphBuilder.CreateUAV(OutDistortionMap);

	FIntPoint LUTResolution = OutDistortionMap->Desc.Extent;
	PassParameters->OverscanOffset = (InDistortionMapWithOverscan->Desc.Extent - OutDistortionMap->Desc.Extent) / 2;

	TShaderMapRef<FCropDistortionDisplacementMapCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CropDistortionDisplacementMap"),
		ComputeShader,
		PassParameters,
		FIntVector(FMath::DivideAndRoundUp(LUTResolution.X, 8), FMath::DivideAndRoundUp(LUTResolution.Y, 8), 1));
}

void FLensDistortionSceneViewExtension::BlendDisplacementMaps_RenderThread(FRDGBuilder& GraphBuilder, const FDisplacementMapBlendingParams& BlendState, ULensDistortionModelHandlerBase* ModelHandler, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMapWithOverscan) const
{
	FBlendDistortionDisplacementMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBlendDistortionDisplacementMapCS::FParameters>();

	// Draw the first distortion map, which should always be valid
	{
		FRDGTextureRef Distortion1 = GraphBuilder.CreateTexture(OutDistortionMapWithOverscan->Desc, TEXT("DistortingDisplacement1"));
		DrawDisplacementMap_RenderThread(GraphBuilder, BlendState.States[0], ModelHandler, InverseOverscan, CameraOverscan, SensorSize, Distortion1);
		PassParameters->InputDistortionMap1 = GraphBuilder.CreateSRV(Distortion1);
	}

	// Draw the second distortion map if any blend is needed
	if (BlendState.BlendType != EDisplacementMapBlendType::OneFocusOneZoom)
	{
		FRDGTextureRef Distortion2 = GraphBuilder.CreateTexture(OutDistortionMapWithOverscan->Desc, TEXT("DistortingDisplacement2"));
		DrawDisplacementMap_RenderThread(GraphBuilder, BlendState.States[1], ModelHandler, InverseOverscan, CameraOverscan, SensorSize, Distortion2);
		PassParameters->InputDistortionMap2 = GraphBuilder.CreateSRV(Distortion2);
	}

	// Draw the 3rd and 4th distortion maps if a 4-way blend is needed
	if (BlendState.BlendType == EDisplacementMapBlendType::TwoFocusTwoZoom)
	{
		FRDGTextureRef Distortion3 = GraphBuilder.CreateTexture(OutDistortionMapWithOverscan->Desc, TEXT("DistortingDisplacement3"));
		FRDGTextureRef Distortion4 = GraphBuilder.CreateTexture(OutDistortionMapWithOverscan->Desc, TEXT("DistortingDisplacement4"));

		DrawDisplacementMap_RenderThread(GraphBuilder, BlendState.States[2], ModelHandler, InverseOverscan, CameraOverscan, SensorSize, Distortion3);
		DrawDisplacementMap_RenderThread(GraphBuilder, BlendState.States[3], ModelHandler, InverseOverscan, CameraOverscan, SensorSize, Distortion4);

		PassParameters->InputDistortionMap3 = GraphBuilder.CreateSRV(Distortion3);
		PassParameters->InputDistortionMap4 = GraphBuilder.CreateSRV(Distortion4);
	}

	PassParameters->OverscanDistortionMap = GraphBuilder.CreateUAV(OutDistortionMapWithOverscan);
	PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FIntPoint DistortionMapResolution = OutDistortionMapWithOverscan->Desc.Extent;
	PassParameters->ThreadIdToUV = FVector2f(1.0f / DistortionMapResolution.X, 1.0f / DistortionMapResolution.Y);

	PassParameters->FxFyScale = FVector2f(BlendState.FxFyScale);

	// Set permutation and blending params based on blend type
	PassParameters->EvalFocus = BlendState.EvalFocus;
	PassParameters->EvalZoom = BlendState.EvalZoom;

	FBlendDistortionDisplacementMapCS::FPermutationDomain PermutationVector;
	if (BlendState.BlendType == EDisplacementMapBlendType::OneFocusOneZoom)
	{
		PermutationVector.Set<FBlendDistortionDisplacementMapCS::FBlendType>(0);
	}
	else if (BlendState.BlendType == EDisplacementMapBlendType::TwoFocusOneZoom)
	{
		PermutationVector.Set<FBlendDistortionDisplacementMapCS::FBlendType>(1);
		PassParameters->PatchCorners[0] = BlendState.PatchCorners[0].ToVector();
		PassParameters->PatchCorners[1] = BlendState.PatchCorners[1].ToVector();
		PassParameters->PatchCorners[2] = FVector4f::Zero();
		PassParameters->PatchCorners[3] = FVector4f::Zero();
	}
	else if (BlendState.BlendType == EDisplacementMapBlendType::OneFocusTwoZoom)
	{
		PermutationVector.Set<FBlendDistortionDisplacementMapCS::FBlendType>(2);
		PassParameters->PatchCorners[0] = BlendState.PatchCorners[0].ToVector();
		PassParameters->PatchCorners[1] = BlendState.PatchCorners[1].ToVector();
		PassParameters->PatchCorners[2] = FVector4f::Zero();
		PassParameters->PatchCorners[3] = FVector4f::Zero();
	}
	else if (BlendState.BlendType == EDisplacementMapBlendType::TwoFocusTwoZoom)
	{
		PermutationVector.Set<FBlendDistortionDisplacementMapCS::FBlendType>(3);
		PassParameters->PatchCorners[0] = BlendState.PatchCorners[0].ToVector();
		PassParameters->PatchCorners[1] = BlendState.PatchCorners[1].ToVector();
		PassParameters->PatchCorners[2] = BlendState.PatchCorners[2].ToVector();
		PassParameters->PatchCorners[3] = BlendState.PatchCorners[3].ToVector();
	}

	TShaderMapRef<FBlendDistortionDisplacementMapCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("BlendDistortionDisplacementMap"),
		ComputeShader,
		PassParameters,
		FIntVector(FMath::DivideAndRoundUp(DistortionMapResolution.X, 8), FMath::DivideAndRoundUp(DistortionMapResolution.Y, 8), 1));
}

void FLensDistortionSceneViewExtension::InvertDistortionMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDistortionMap, float InInverseOverscan, FRDGTextureRef& OutUndistortionMap) const
{
	FInvertDisplacementParameters* PassParameters = GraphBuilder.AllocParameters<FInvertDisplacementParameters>();

	FScreenPassRenderTarget Output;
	Output.Texture = OutUndistortionMap;
	Output.ViewRect = FIntRect(FIntPoint(0, 0), OutUndistortionMap->Desc.Extent);
	Output.LoadAction = ERenderTargetLoadAction::EClear;
	Output.UpdateVisualizeTextureExtent();
	
	const int32 NumSquares = FMath::Clamp(CVarLensDistortionInvertGridDensity.GetValueOnRenderThread(), 64, 255);
	FIntPoint GridDimensions = FIntPoint(NumSquares);

	// Scale the grid density by the overscan to ensure that there is no change in number of distorted vertices
	GridDimensions = FIntPoint(FMath::CeilToInt(GridDimensions.X * InInverseOverscan), FMath::CeilToInt(GridDimensions.Y * InInverseOverscan));
	
	PassParameters->GridDimensions = GridDimensions;

	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	PassParameters->DistortionMap = GraphBuilder.CreateSRV(InDistortionMap);
	PassParameters->DistortionMapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PassParameters->OverscanFactor = float(InDistortionMap->Desc.Extent.X) / float(OutUndistortionMap->Desc.Extent.X);
	PassParameters->PixelToUV = FVector2f(1.0f) / FVector2f(OutUndistortionMap->Desc.Extent);
	PassParameters->PixelToOverscanUV = FVector2f(1.0f) / FVector2f(InDistortionMap->Desc.Extent);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("InvertDistortionDisplacementMap"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, GridDimensions, Output](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(Output.ViewRect.Min.X, Output.ViewRect.Min.Y, 0.0f, Output.ViewRect.Max.X, Output.ViewRect.Max.Y, 1.0f);

			TShaderMapRef<FInvertDisplacementVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
			TShaderMapRef<FInvertDisplacementPS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

			SetScreenPassPipelineState(RHICmdList, FScreenPassPipelineState(VertexShader, PixelShader));
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);

			// No vertex buffer is needed because we compute it in the VS
			RHICmdList.SetStreamSource(0, NULL, 0);

			// The following code for setting up this index buffer based on FTesselatedScreenRectangleIndexBuffer::InitRHI()
			TArray<uint32> IndexBuffer;

			const uint32 Width = GridDimensions.X;
			const uint32 Height = GridDimensions.Y;
			const uint32 NumVertices = (Width + 1) * (Height + 1);
			const uint32 NumTriangles = Width * Height * 2;
			const uint32 NumIndices = NumTriangles * 3;

			IndexBuffer.AddUninitialized(NumIndices);

			uint32* IndexBufferData = (uint32*)IndexBuffer.GetData();

			for (uint32 IndexY = 0; IndexY < Height; ++IndexY)
			{
				for (uint32 IndexX = 0; IndexX < Width; ++IndexX)
				{
					// left top to bottom right in reading order
					uint32 Index00 = IndexX + IndexY * (Width + 1);
					uint32 Index10 = Index00 + 1;
					uint32 Index01 = Index00 + (Width + 1);
					uint32 Index11 = Index01 + 1;

					// triangle A
					*IndexBufferData++ = Index00;
					*IndexBufferData++ = Index01;
					*IndexBufferData++ = Index10;

					// triangle B
					*IndexBufferData++ = Index11;
					*IndexBufferData++ = Index10;
					*IndexBufferData++ = Index01;
				}
			}

			// Create index buffer. Fill buffer with initial data upon creation
			FBufferRHIRef IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("InvertDistortionMapIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(IndexBuffer));

			RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, NumVertices, 0, NumTriangles, 1);
		});
}

void FLensDistortionSceneViewExtension::FillSTDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InUndisplacementMap, float InOverscan, FRDGTextureRef& OutFilledDisplacementMap) const
{
	FFillSTDisplacementMapParameters* PassParameters = GraphBuilder.AllocParameters<FFillSTDisplacementMapParameters>();

	FScreenPassRenderTarget Output;
	Output.Texture = OutFilledDisplacementMap;
	Output.ViewRect = FIntRect(FIntPoint(0, 0), OutFilledDisplacementMap->Desc.Extent);
	Output.LoadAction = ERenderTargetLoadAction::EClear;
	Output.UpdateVisualizeTextureExtent();
	
	const int32 NumSquares = FMath::Clamp(CVarLensDistortionInvertGridDensity.GetValueOnRenderThread(), 64, 255);
	const FIntPoint GridDimensions = FIntPoint(NumSquares);

	PassParameters->GridDimensions = GridDimensions;

	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	PassParameters->UndisplacementMap = GraphBuilder.CreateSRV(InUndisplacementMap);
	PassParameters->UndisplacementMapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	
	PassParameters->Overscan = InOverscan;
	PassParameters->PixelToUV = FVector2f(1.0f) / FVector2f(InUndisplacementMap->Desc.Extent);
	PassParameters->PixelToOverscannedUV = FVector2f(1.0f) / FVector2f(OutFilledDisplacementMap->Desc.Extent);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("FillSTDisplacementMap"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, GridDimensions, Output](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(Output.ViewRect.Min.X, Output.ViewRect.Min.Y, 0.0f, Output.ViewRect.Max.X, Output.ViewRect.Max.Y, 1.0f);

			TShaderMapRef<FFillSTDisplacementMapVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
			TShaderMapRef<FFillSTDisplacementMapPS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

			SetScreenPassPipelineState(RHICmdList, FScreenPassPipelineState(VertexShader, PixelShader));
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);

			// No vertex buffer is needed because we compute it in the VS
			RHICmdList.SetStreamSource(0, NULL, 0);

			// The following code for setting up this index buffer based on FTesselatedScreenRectangleIndexBuffer::InitRHI()
			TArray<uint32> IndexBuffer;

			const uint32 Width = GridDimensions.X;
			const uint32 Height = GridDimensions.Y;
			const uint32 NumVertices = (Width + 1) * (Height + 1);
			const uint32 NumTriangles = Width * Height * 2;
			const uint32 NumIndices = NumTriangles * 3;

			IndexBuffer.AddUninitialized(NumIndices);

			uint32* IndexBufferData = (uint32*)IndexBuffer.GetData();

			for (uint32 IndexY = 0; IndexY < Height; ++IndexY)
			{
				for (uint32 IndexX = 0; IndexX < Width; ++IndexX)
				{
					// left top to bottom right in reading order
					uint32 Index00 = IndexX + IndexY * (Width + 1);
					uint32 Index10 = Index00 + 1;
					uint32 Index01 = Index00 + (Width + 1);
					uint32 Index11 = Index01 + 1;

					// triangle A
					*IndexBufferData++ = Index00;
					*IndexBufferData++ = Index01;
					*IndexBufferData++ = Index10;

					// triangle B
					*IndexBufferData++ = Index11;
					*IndexBufferData++ = Index10;
					*IndexBufferData++ = Index01;
				}
			}

			// Create index buffer. Fill buffer with initial data upon creation
			FBufferRHIRef IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("InvertDistortionMapIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(IndexBuffer));

			RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, NumVertices, 0, NumTriangles, 1);
		});
}

void FLensDistortionSceneViewExtension::RecenterSTDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDisplacementMap, float InOverscan, FRDGTextureRef& OutRecenteredDisplacementMap) const
{
	FScreenPassRenderTarget Output;
	Output.Texture = OutRecenteredDisplacementMap;
	Output.ViewRect = FIntRect(FIntPoint(0, 0), OutRecenteredDisplacementMap->Desc.Extent);
	Output.LoadAction = ERenderTargetLoadAction::EClear;
	Output.UpdateVisualizeTextureExtent();
	
	FRecenterSTDisplacementMapPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRecenterSTDisplacementMapPS::FParameters>();
	PassParameters->STMapInvSize = FVector2f(1.0f / InDisplacementMap->Desc.Extent.X, 1.0f / InDisplacementMap->Desc.Extent.Y);
	PassParameters->STMap = GraphBuilder.CreateSRV(InDisplacementMap);
	PassParameters->STMapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->Overscan = InOverscan;
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FScreenPassVS> ScreenPassVS(GlobalShaderMap);
	TShaderMapRef<FRecenterSTDisplacementMapPS> DistortImageShader(GlobalShaderMap);

	FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	const FScreenPassTextureViewport OutViewport(OutRecenteredDisplacementMap->Desc.Extent);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RecenterSTDisplacementMap"),
		PassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[ScreenPassVS, DistortImageShader, OutViewport, PassParameters, DefaultBlendState](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		DrawScreenPass(
			RHICmdList,
			FScreenPassViewInfo(),
			OutViewport,
			OutViewport,
			FScreenPassPipelineState(ScreenPassVS, DistortImageShader, DefaultBlendState),
			EScreenPassDrawFlags::None,
			[&](FRHICommandList&)
		{
			SetShaderParameters(RHICmdList, DistortImageShader, DistortImageShader.GetPixelShader(), *PassParameters);
		});
	});
}


int32 FLensDistortionSceneViewExtension::GetPriority() const
{
	return GetDefault<UCameraCalibrationSettings>()->GetSceneViewExtensionPriority();
}

void FLensDistortionSceneViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	FLensDistortionLUT ViewDistortionLUT;

	if (RenderViewDistortionLUT(GraphBuilder, InView.ViewActor.ActorUniqueId, ViewDistortionLUT))
	{
		LensDistortion::SetLUTUnsafe(InView, ViewDistortionLUT);
	}
}

bool FLensDistortionSceneViewExtension::RenderViewDistortionLUT(FRDGBuilder& GraphBuilder, uint32 InViewActorID, FLensDistortionLUT& ViewDistortionLUT) const
{
	FScopeLock ScopeLock(&DistortionStateMapCriticalSection);
	const FCameraDistortionProxy* Proxy = DistortionStateMap.Find(InViewActorID);
	if (!Proxy)
	{
		return false;
	}

	if (Proxy->RenderingMode != EDistortionRenderingMode::TemporalSuperResolution)
	{
		return false;
	}

	FVector2D SensorSize = FVector2D(Proxy->FilmbackSettings.SensorWidth, Proxy->FilmbackSettings.SensorHeight);
	FDisplacementMapBlendingParams BlendState = Proxy->Params;

	TStrongObjectPtr<ULensDistortionModelHandlerBase> ModelHandler = Proxy->LensDistortionHandler.Pin();
	
	bool bIsSTMap = false;
	bool bForwardDistort = true;
	float RawInverseOverscan = 1.0f;
	FTextureRHIRef UndistortionMapTextureRef = nullptr;
	FTextureRHIRef DistortionMapTextureRef = nullptr;
	
	if (ModelHandler.IsValid())
	{
		if (!Proxy->Params.States[0].DistortionInfo.Parameters.Num())
		{
			bIsSTMap = true;

			UndistortionMapTextureRef = ModelHandler->GetDistortionDisplacementMap()->GetResource()->GetTexture2DRHI();
			DistortionMapTextureRef = ModelHandler->GetUndistortionDisplacementMap()->GetResource()->GetTexture2DRHI();
		}
		
		bForwardDistort = ModelHandler->IsForwardDistorting();
		RawInverseOverscan = bForwardDistort ? ModelHandler->ComputeOverscanFactor() : ModelHandler->ComputeInverseOverscanFactor();
	}
	
	if (bIsSTMap)
	{
		// ST Maps generally have their top left corner corresponding to the distortion at the original frustum, not the overscanned frustum, which means that
		// for cases where the distortion would require overscanned pixels, we must generate those distortion values for the distortion map from the undistortion map
		// Further, ST maps may have intrinsic inaccuracies that can cause artifacting in TSR, so use a warp grid inversion to generate a fully approximate
		// distortion ST map from the undistortion map. We also need to recenter the undistortion map to correspond to the overscan, as well as scaling its displacement
		// by the overscan amount
		FRDGTextureRef OriginalUndistortionMap = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(UndistortionMapTextureRef, TEXT("OriginalUndistortionDisplacementMap")));

		FRDGTextureRef OriginalDistortionMap = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DistortionMapTextureRef, TEXT("OriginalDistortionDisplacementMap")));

		FRDGTextureDesc FilledDistortionMapDesc = FRDGTextureDesc::Create2D(
			OriginalDistortionMap->Desc.Extent,
			PF_G32R32F,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_RenderTargetable);

		FRDGTextureDesc FilledUndistortionMapDesc = FRDGTextureDesc::Create2D(
			OriginalUndistortionMap->Desc.Extent,
			PF_G32R32F,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_RenderTargetable);
		
		ViewDistortionLUT.DistortingDisplacementTexture = GraphBuilder.CreateTexture(FilledDistortionMapDesc, TEXT("DistortionDisplacementMap"));
		ViewDistortionLUT.UndistortingDisplacementTexture = GraphBuilder.CreateTexture(FilledUndistortionMapDesc, TEXT("UndistortionDisplacementMap"));

		// If there is any overscan, we will need to fill in the overscanned area of the distortion LUT because for ST Maps, the top left corner
		// of the map corresponds to the top left corner of the original frustum. Perform a mesh warp to invert the undistortion map
		FRDGTextureRef ForwardTexture = Proxy->CameraOverscan > 1.0f ? OriginalUndistortionMap : OriginalDistortionMap;
		FRDGTextureRef ForwardTextureOut = Proxy->CameraOverscan > 1.0f ? ViewDistortionLUT.UndistortingDisplacementTexture : ViewDistortionLUT.DistortingDisplacementTexture;
		FRDGTextureRef InverseTextureOut = Proxy->CameraOverscan > 1.0f ? ViewDistortionLUT.DistortingDisplacementTexture : ViewDistortionLUT.UndistortingDisplacementTexture;

		// Fudge the overscan just a little bit to ensure that the edge of the ST map is not visible during the TSR distortion, as that can cause ghosting and artifacts
		const float OverscanAdjustment = 0.98f;
		FillSTDisplacementMap_RenderThread(GraphBuilder, ForwardTexture, Proxy->CameraOverscan * OverscanAdjustment, InverseTextureOut);
		RecenterSTDisplacementMap_RenderThread(GraphBuilder, ForwardTexture, Proxy->CameraOverscan * OverscanAdjustment, ForwardTextureOut);

		ViewDistortionLUT.DistortionOverscan = Proxy->CameraOverscan * OverscanAdjustment;
		ViewDistortionLUT.DistortionGridDimensions = FIntPoint(32 * Proxy->CameraOverscan, 20 * Proxy->CameraOverscan);
		
		return true;
	}
	
	ScopeLock.Unlock();

	// Create the distortion map and undistortion map textures for the FLensDistortionLUT for this frame
	const FIntPoint DisplacementMapResolution = FIntPoint(256, 256);

	// Define forward and reverse distortion texture descriptors (matching original logic)
	FRDGTextureDesc ForwardDistortionMapDesc = FRDGTextureDesc::Create2D(
		DisplacementMapResolution,
		PF_G32R32F,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureDesc InverseDistortionMapDesc = FRDGTextureDesc::Create2D(
		DisplacementMapResolution,
		PF_G32R32F,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_RenderTargetable);
		
	// Choose texture descriptors based on distortion direction
	FRDGTextureDesc DistortionMapDesc = bForwardDistort ? ForwardDistortionMapDesc : InverseDistortionMapDesc;
	FRDGTextureDesc UndistortionMapDesc = bForwardDistort ? InverseDistortionMapDesc : ForwardDistortionMapDesc;
	
	ViewDistortionLUT.DistortingDisplacementTexture = GraphBuilder.CreateTexture(DistortionMapDesc, TEXT("DistortionDisplacementMap"));
	ViewDistortionLUT.UndistortingDisplacementTexture = GraphBuilder.CreateTexture(UndistortionMapDesc, TEXT("UndistortionDisplacementMap"));
	float InverseOverscan = FMath::Clamp(RawInverseOverscan, 0.0f, 2.0f);
	
	// Adjust the overscan resolution to be square, with each side being a multiple of 8
	const FIntPoint OverscanResolution = FIntPoint(FMath::CeilToInt(InverseOverscan * 32) * 8);
	InverseOverscan = float(OverscanResolution.X) / float(DisplacementMapResolution.X);

	// Create the texture for the overscanned distortion map
	FRDGTextureDesc OverscanDesc = FRDGTextureDesc::Create2D(
		OverscanResolution,
		PF_G32R32F,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef DistortionMapWithOverscan = GraphBuilder.CreateTexture(OverscanDesc, TEXT("DistortionMapWithOverscan"));

	// Generate distortion map through handler, then invert it to create undistortion map
	// Anamorphic model uses a precise equation for its _undistort_ map instead of its distort map, so draw the undistort map first and invert it to find the distort map
	FRDGTextureRef ForwardDistortionMap = bForwardDistort ? ViewDistortionLUT.DistortingDisplacementTexture : ViewDistortionLUT.UndistortingDisplacementTexture;
	FRDGTextureRef InverseDistortionMap = bForwardDistort ? ViewDistortionLUT.UndistortingDisplacementTexture : ViewDistortionLUT.DistortingDisplacementTexture;
	
	BlendDisplacementMaps_RenderThread(GraphBuilder, BlendState, ModelHandler.Get(), InverseOverscan, Proxy->CameraOverscan, SensorSize, DistortionMapWithOverscan);
	InvertDistortionMap_RenderThread(GraphBuilder, DistortionMapWithOverscan, InverseOverscan, InverseDistortionMap);
	CropDisplacementMap_RenderThread(GraphBuilder, DistortionMapWithOverscan, ForwardDistortionMap);

	return true;
}

void FLensDistortionSceneViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (PassId == EPostProcessingPass::Tonemap)
	{
		FScopeLock ScopeLock(&DistortionStateMapCriticalSection);

		const FCameraDistortionProxy* Proxy = DistortionStateMap.Find(InView.ViewActor.ActorUniqueId);
		
		if (Proxy && Proxy->RenderingMode == EDistortionRenderingMode::PassAfterTonemap)
		{
			TStrongObjectPtr<ULensDistortionModelHandlerBase> ModelHandler = Proxy->LensDistortionHandler.Pin();
			
			if (ModelHandler.IsValid())
			{
				UMaterialInterface* LensDistortionMaterial = ModelHandler->GetDistortionMID();

				InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FLensDistortionSceneViewExtension::PostProcessPassAfterTonemap_RenderThread, LensDistortionMaterial));
			}
		}
	}
}

FScreenPassTexture FLensDistortionSceneViewExtension::PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessMaterialInputs& InOutInputs, UMaterialInterface* LensDistortionMaterial)
{
	if (IsValid(LensDistortionMaterial))
	{
		// Note: Plugin material could be entirely replaced by shaders should we deprecate the blendable EDistortionRenderingMode::PostProcessMaterial path.
		return AddPostProcessMaterialPass(GraphBuilder, InView, InOutInputs, LensDistortionMaterial);
	}
	else
	{
		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
}
