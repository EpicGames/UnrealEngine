// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensFile.h"

#include "Algo/MaxElement.h"
#include "CalibratedMapProcessor.h"
#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationSubsystem.h"
#include "CineCameraComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensFileObjectVersion.h"
#include "LensFileRendering.h"
#include "LensInterpolationUtils.h"
#include "Curves/CurveEvaluation.h"
#include "Models/SphericalLensModel.h"
#include "Tables/BaseLensTable.h"
#include "Tables/LensTableUtils.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensFile)

namespace LensFileUtils
{
	UTextureRenderTarget2D* CreateDisplacementMapRenderTarget(UObject* Outer, const FIntPoint DisplacementMapResolution)
	{
		check(Outer);

		UTextureRenderTarget2D* NewRenderTarget2D = NewObject<UTextureRenderTarget2D>(Outer, MakeUniqueObjectName(Outer, UTextureRenderTarget2D::StaticClass(), TEXT("LensDisplacementMap")), RF_Public);
		NewRenderTarget2D->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RG16f;
		NewRenderTarget2D->ClearColor = FLinearColor(0.5f, 0.5f, 0.5f, 0.5f);
		NewRenderTarget2D->Filter = TF_Bilinear;
		NewRenderTarget2D->AddressX = TA_Clamp;
		NewRenderTarget2D->AddressY = TA_Clamp;
		NewRenderTarget2D->bAutoGenerateMips = false;
		NewRenderTarget2D->bCanCreateUAV = true;
		NewRenderTarget2D->InitAutoFormat(DisplacementMapResolution.X, DisplacementMapResolution.Y);
		NewRenderTarget2D->UpdateResourceImmediate(true);

		//Flush RHI thread after creating texture render target to make sure that RHIUpdateTextureReference is executed before doing any rendering with it
		//This makes sure that Value->TextureReference.TextureReferenceRHI->GetReferencedTexture() is valid so that FUniformExpressionSet::FillUniformBuffer properly uses the texture for rendering, instead of using a fallback texture
		ENQUEUE_RENDER_COMMAND(FlushRHIThreadToUpdateTextureRenderTargetReference)(
			[](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			});

		return NewRenderTarget2D;
	}

	float EvalAtTwoPoints(const float EvalTime, const float Time0, const float Time1, const float Value0, const float Value1, const float Tangent0, const float Tangent1)
	{
		if (FMath::IsNearlyEqual(Time0, Time1))
		{
			return Value0;
		}

		constexpr float OneThird = 1.0f / 3.0f;

		const float CurveDiff = Time1 - Time0;
		const float CurveAlpha = (EvalTime - Time0) / CurveDiff;

		const float DeltaInput = Value1 - Value0;
		const float CurveDelta = DeltaInput / CurveDiff;
		const float CurveTan0 = Tangent0 * CurveDelta;
		const float CurveTan1 = Tangent1 * CurveDelta;

		const float P0 = Value0;
		const float P3 = Value1;
		const float P1 = P0 + (CurveTan0 * CurveDiff * OneThird);
		const float P2 = P3 - (CurveTan1 * CurveDiff * OneThird);
		return UE::Curves::BezierInterp(P0, P1, P2, P3, CurveAlpha);
	}

	void FindWeightsAndInterp(float InEvalTime, TConstArrayView<float> InTimes, TConstArrayView<float> InTangents, TOptional<float> LerpFactor, TConstArrayView<float> Inputs, float& Output)
	{
		const int32 CurveCount = InTimes.Num();
		check(CurveCount == 2 || CurveCount == 4);

		const int32 ResultCount = InTimes.Num() / 2;

		TArray<float, TInlineAllocator<4>> BezierResults;
		BezierResults.SetNumZeroed(ResultCount);

		for (int32 CurveIndex = 0; CurveIndex < InTimes.Num(); CurveIndex += 2)
		{
			BezierResults[CurveIndex / 2] = EvalAtTwoPoints(InEvalTime
				, InTimes[CurveIndex + 0]
				, InTimes[CurveIndex + 1]
				, Inputs[CurveIndex + 0]
				, Inputs[CurveIndex + 1]
				, InTangents[CurveIndex + 0]
				, InTangents[CurveIndex + 1]);
		}

		if (LerpFactor.IsSet())
		{
			check(BezierResults.Num() == 2);

			const float BlendFactor = LerpFactor.GetValue();
			Output = FMath::Lerp(BezierResults[0], BezierResults[1], BlendFactor);
		}
		else
		{
			check(BezierResults.Num() == 1);
			Output = BezierResults[0];
		}
	}

	void FindWeightsAndInterp(float InEvalTime, TConstArrayView<float> InTimes, TConstArrayView<float> InTangents, TOptional<float> LerpFactor, TConstArrayView<TConstArrayView<float>> Inputs, TArray<float>& Output)
	{
		const int32 CurveCount = InTimes.Num();
		check(CurveCount == 2 || CurveCount == 4);
		
		constexpr float OneThird = 1.0f / 3.0f;
		const int32 ResultCount = InTimes.Num() / 2;
		const int32 InputCount = Inputs[0].Num();
		
		TArray<TArray<float, TInlineAllocator<10>>, TInlineAllocator<4>> BezierResults;
		BezierResults.SetNumZeroed(ResultCount);
		for(TArray<float, TInlineAllocator<10>>& Result: BezierResults)
		{
			Result.SetNumZeroed(InputCount);
		}
        
		for(int32 CurveIndex = 0; CurveIndex < InTimes.Num(); CurveIndex += 2)
		{
			TArray<float, TInlineAllocator<10>>& ResultContainer = BezierResults[CurveIndex/2];

			TConstArrayView<float> Inputs0 = Inputs[CurveIndex+0];
			TConstArrayView<float> Inputs1 = Inputs[CurveIndex+1];

			for(int32 InputIndex = 0; InputIndex < Inputs0.Num(); ++InputIndex)
			{
				ResultContainer[InputIndex] = EvalAtTwoPoints(InEvalTime
					, InTimes[CurveIndex + 0]
					, InTimes[CurveIndex + 1]
					, Inputs0[InputIndex]
					, Inputs1[InputIndex]
					, InTangents[CurveIndex + 0]
					, InTangents[CurveIndex + 1]);
			}
		}

		if(LerpFactor.IsSet())
		{
			check(BezierResults.Num() == 2);

			const float BlendFactor = LerpFactor.GetValue();
			Output.SetNumUninitialized(InputCount);
			for(int32 InputIndex = 0; InputIndex < BezierResults[0].Num(); ++InputIndex)
			{
				Output[InputIndex] = FMath::Lerp(BezierResults[0][InputIndex], BezierResults[1][InputIndex], BlendFactor);
			}
		}
		else
		{
			check(BezierResults.Num() == 1);
			Output = MoveTemp(BezierResults[0]);
		}
	}
}

const TArray<FVector2D> ULensFile::UndistortedUVs =
{
	FVector2D(0.0f, 0.0f),
	FVector2D(0.5f, 0.0f),
	FVector2D(1.0f, 0.0f),
	FVector2D(1.0f, 0.5f),
	FVector2D(1.0f, 1.0f),
	FVector2D(0.5f, 1.0f),
	FVector2D(0.0f, 1.0f),
	FVector2D(0.0f, 0.5f)
};

ULensFile::ULensFile()
	: FTickableGameObject(ETickableTickType::Never)
{
	LensInfo.LensModel = USphericalLensModel::StaticClass();
	
	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		CalibratedMapProcessor = MakeUnique<FCalibratedMapProcessor>();
#if WITH_EDITOR
		UCameraCalibrationSettings* DefaultSettings = GetMutableDefault<UCameraCalibrationSettings>();
		DefaultSettings->OnDisplacementMapResolutionChanged().AddUObject(this, &ULensFile::UpdateDisplacementMapResolution);
		DefaultSettings->OnCalibrationInputToleranceChanged().AddUObject(this, &ULensFile::UpdateInputTolerance);

		UpdateInputTolerance(DefaultSettings->GetCalibrationInputTolerance());
#endif // WITH_EDITOR
	}
}

void ULensFile::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FLensFileObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FLensFileObjectVersion::GUID) < FLensFileObjectVersion::EditableFocusCurves)
		{
			BuildLensTableFocusCurves();
		}
	}
#endif 
}

#if WITH_EDITOR

void ULensFile::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.Property != nullptr)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		FName ActiveMemberName;
		FProperty* ActiveMemberProperty = nullptr;
		if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode() && PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue())
		{
			ActiveMemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
			ActiveMemberName = ActiveMemberProperty->GetFName();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FSTMapInfo, DistortionMap))
		{
			//When the distortion map (stmap) changes, flag associated derived data as dirty to update it
			if (ActiveMemberProperty)
			{
				//@todo Find out which map was changed and set it dirty
			}
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensInfo, LensModel))
		{
			OnLensFileModelChangedDelegate.Broadcast(LensInfo.LensModel);

			//LensModel has changed, clear distortion and focal length tables
			LensDataTableUtils::EmptyTable(DistortionTable);
			LensDataTableUtils::EmptyTable(FocalLengthTable);
		}
		else if (ActiveMemberName == GET_MEMBER_NAME_CHECKED(ULensFile, LensInfo))
		{
			//Make sure sensor dimensions have valid values
			LensInfo.SensorDimensions.X = FMath::Max(LensInfo.SensorDimensions.X, 1.0f);
			LensInfo.SensorDimensions.Y = FMath::Max(LensInfo.SensorDimensions.Y, 1.0f);
		}
	}
	
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

#endif //WITH_EDITOR

bool ULensFile::EvaluateDistortionParameters(float InFocus, float InZoom, FDistortionInfo& OutEvaluatedValue) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateDistortionParameters);

	using FBlendParams = LensInterpolationUtils::FDistortionMapBlendParams<FDistortionTable>;
	using FBlendResults = LensInterpolationUtils::FDistortionMapBlendResults;
	
	FBlendParams Params;
	Params.bGenerateBlendedDistortionParams = true;
	Params.GetInterpolatedImageCenter = FBlendParams::FGetInterpolatedImageCenter::CreateUObject(this, &ULensFile::EvaluateImageCenterParameters);
	Params.GetInterpolatedFocalLength = FBlendParams::FGetInterpolatedFocalLength::CreateUObject(this, &ULensFile::EvaluateFocalLength);

	LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(Params.DefaultDistortionParams);

	FBlendResults Results = LensInterpolationUtils::DistortionMapBlend(DistortionTable, InFocus, InZoom, Params);
	if (!Results.bValid || !Results.BlendedDistortionParams.IsSet())
	{
		return false;
	}

	OutEvaluatedValue = Results.BlendedDistortionParams.GetValue();
	return true;
}

bool ULensFile::EvaluateFocalLength(float InFocus, float InZoom, FFocalLengthInfo& OutEvaluatedValue) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateFocalLength);

	constexpr int32 NumParams = 2;
	TArray<float> BlendedParameters;
	if (LensInterpolationUtils::IndexedParameterBlend(FocalLengthTable.FocusPoints, FocalLengthTable.FocusCurves, InFocus, InZoom, NumParams, BlendedParameters))
	{
		ensure(BlendedParameters.Num() == NumParams);
		OutEvaluatedValue.FxFy.X = BlendedParameters[FFocalLengthTable::FParameters::Fx];
		OutEvaluatedValue.FxFy.Y = BlendedParameters[FFocalLengthTable::FParameters::Fy];

		return true;
	}
	
	return false;
}

bool ULensFile::EvaluateImageCenterParameters(float InFocus, float InZoom, FImageCenterInfo& OutEvaluatedValue) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateImageCenterParameters);

	constexpr int32 NumParams = 2;
	TArray<float> BlendedParameters;
	if (LensInterpolationUtils::IndexedParameterBlend(ImageCenterTable.FocusPoints, ImageCenterTable.FocusCurves, InFocus, InZoom, NumParams, BlendedParameters))
	{
		ensure(BlendedParameters.Num() == NumParams);
		OutEvaluatedValue.PrincipalPoint.X = BlendedParameters[FImageCenterTable::FParameters::Cx];
		OutEvaluatedValue.PrincipalPoint.Y = BlendedParameters[FImageCenterTable::FParameters::Cy];

		return true;
	}
	
	return false;
}

bool ULensFile::EvaluateDistortionData(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* InLensHandler) const
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateDistortionData);

	if (InLensHandler == nullptr)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Can't evaluate LensFile '%ls' - Invalid Lens Handler", *GetName());
		return false;
	}
	
	if (InLensHandler->GetUndistortionDisplacementMap() == nullptr)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Can't evaluate LensFile '%ls' - Invalid undistortion displacement map in LensHandler '%ls'", *GetName(), *InLensHandler->GetName());
		return false;
	}

	if (InLensHandler->GetDistortionDisplacementMap() == nullptr)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Can't evaluate LensFile '%ls' - Invalid distortion displacement map in LensHandler '%ls'", *GetName(), *InLensHandler->GetName());
		return false;
	}

	if (LensInfo.LensModel == nullptr)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Can't evaluate LensFile '%ls' - Invalid Lens Model", *GetName());
		SetupNoDistortionOutput(InLensHandler);
		return false;
	}

	if (InLensHandler->IsModelSupported(LensInfo.LensModel) == false)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, "Can't evaluate LensFile '%ls' - LensHandler '%ls' doesn't support lens model '%ls'", *GetName(), *InLensHandler->GetName(), *LensInfo.LensModel.GetDefaultObject()->GetModelName().ToString());
		SetupNoDistortionOutput(InLensHandler);
		return false;
	}
	
	if(DataMode == ELensDataMode::Parameters)
	{
		return EvaluateDistortionForParameters(InFocus, InZoom, InFilmback, InLensHandler);
	}
	else
	{
		//Only other mode for now
		check(DataMode == ELensDataMode::STMap);

		return EvaluateDistortionForSTMaps(InFocus, InZoom, InFilmback, InLensHandler);
	}
}

float ULensFile::ComputeOverscan(const FDistortionData& DerivedData, FVector2D PrincipalPoint) const
{
	//Edge case if computed data hasn't came back yet
	if (UndistortedUVs.Num() != DerivedData.DistortedUVs.Num())
	{
		return 1.0f;
	}

	TArray<float, TInlineAllocator<8>> OverscanFactors;
	OverscanFactors.Reserve(UndistortedUVs.Num());
	for (int32 Index = 0; Index < UndistortedUVs.Num(); ++Index)
	{
		const FVector2D& UndistortedUV = UndistortedUVs[Index];
		const FVector2D& DistortedUV = DerivedData.DistortedUVs[Index] + (PrincipalPoint - FVector2D(0.5f, 0.5f)) * 2.0f;
		const float OverscanX = (UndistortedUV.X != 0.5f) ? (DistortedUV.X - 0.5f) / (UndistortedUV.X - 0.5f) : 1.0f;
		const float OverscanY = (UndistortedUV.Y != 0.5f) ? (DistortedUV.Y - 0.5f) / (UndistortedUV.Y - 0.5f) : 1.0f;
		OverscanFactors.Add(FMath::Max(OverscanX, OverscanY));
	}

	float* MaxOverscanFactor = Algo::MaxElement(OverscanFactors);
	const float FoundOverscan = MaxOverscanFactor ? *MaxOverscanFactor : 1.0f;
	
	return FoundOverscan;
}

void ULensFile::SetupNoDistortionOutput(ULensDistortionModelHandlerBase* LensHandler) const
{
	LensFileRendering::ClearDisplacementMap(LensHandler->GetUndistortionDisplacementMap());
	LensFileRendering::ClearDisplacementMap(LensHandler->GetDistortionDisplacementMap());
	LensHandler->SetOverscanFactor(1.0f);
}

void ULensFile::GetBlendState(float InFocus, float InZoom, FVector2D InFilmback, FDisplacementMapBlendingParams& OutBlendState)
{
	using FBlendParams = LensInterpolationUtils::FDistortionMapBlendParams<FDistortionTable>;
	using FBlendResults = LensInterpolationUtils::FDistortionMapBlendResults;
	
	FBlendParams Params;
	Params.bGenerateBlendingParams = true;
	Params.GetInterpolatedImageCenter = FBlendParams::FGetInterpolatedImageCenter::CreateUObject(this, &ULensFile::EvaluateImageCenterParameters);
	Params.GetInterpolatedFocalLength = FBlendParams::FGetInterpolatedFocalLength::CreateUObject(this, &ULensFile::EvaluateFocalLength);

	FBlendResults Results = LensInterpolationUtils::DistortionMapBlend(DistortionTable, InFocus, InZoom, Params);

	if (Results.bValid && Results.BlendingParams.IsSet())
	{
		OutBlendState = Results.BlendingParams.GetValue();
		OutBlendState.FxFyScale = FVector2D(InFilmback.X / LensInfo.SensorDimensions.X, InFilmback.Y / LensInfo.SensorDimensions.Y);
	}
}

bool ULensFile::EvaluateDistortionForParameters(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* InLensHandler) const
{
	using FBlendParams = LensInterpolationUtils::FDistortionMapBlendParams<FDistortionTable>;
	using FBlendResults = LensInterpolationUtils::FDistortionMapBlendResults;
	
	FCameraFilmbackSettings CameraFilmback;
	CameraFilmback.SensorWidth = InFilmback.X;
	CameraFilmback.SensorHeight = InFilmback.Y;

	const FVector2D FxFyScale = FVector2D(LensInfo.SensorDimensions.X / CameraFilmback.SensorWidth, LensInfo.SensorDimensions.Y / CameraFilmback.SensorHeight);
	
	// Compute interpolated image center and focal length to pass to the lens handler
	FLensDistortionState InterpolatedState;
	EvaluateImageCenterParameters(InFocus, InZoom, InterpolatedState.ImageCenter);
	EvaluateFocalLength(InFocus, InZoom, InterpolatedState.FocalLengthInfo);
	InterpolatedState.FocalLengthInfo.FxFy *= FxFyScale;
	
	FBlendParams Params;
	Params.bGenerateBlendingParams = true;
	Params.bGenerateBlendedDistortionParams = true;
	Params.bGenerateDistortionMaps = true;
	Params.bGenerateBlendedOverscan = true;
	Params.UndistortedMaps = UndistortionDisplacementMapHolders;
	Params.DistortedMaps = DistortionDisplacementMapHolders;
	Params.GetInterpolatedImageCenter = FBlendParams::FGetInterpolatedImageCenter::CreateUObject(this, &ULensFile::EvaluateImageCenterParameters);
	Params.GetInterpolatedFocalLength = FBlendParams::FGetInterpolatedFocalLength::CreateUObject(this, &ULensFile::EvaluateFocalLength);
	
	LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(Params.DefaultDistortionParams);
	
	Params.ProcessDisplacementMaps = FBlendParams::FProcessDisplacementMaps::CreateLambda(
		[InLensHandler, &CameraFilmback, &FxFyScale](const FLensDistortionState& InDistortionState, UTextureRenderTarget2D* InUndistortionMap, UTextureRenderTarget2D* InDistortionMap)
		{
			FLensDistortionState FinalDistortionState(InDistortionState);
			FinalDistortionState.FocalLengthInfo.FxFy *= FxFyScale;

			InLensHandler->SetDistortionState(FinalDistortionState);
			InLensHandler->SetCameraFilmback(CameraFilmback);
			
			InLensHandler->DrawUndistortionDisplacementMap(InUndistortionMap);
			InLensHandler->DrawDistortionDisplacementMap(InDistortionMap);
		}
	);

	Params.GetOverscan = FBlendParams::FGetOverscan::CreateLambda(
		[InLensHandler, &CameraFilmback, &FxFyScale]
		(const FDistortionFocusPoint& FocusPoint, float Zoom, const FLensDistortionState& InDistortionState)
		{
			FLensDistortionState FinalDistortionState(InDistortionState);
			FinalDistortionState.FocalLengthInfo.FxFy *= FxFyScale;

			InLensHandler->SetDistortionState(FinalDistortionState);
			InLensHandler->SetCameraFilmback(CameraFilmback);

			return InLensHandler->ComputeOverscanFactor();
		}
	);
	
	FBlendResults Results = LensInterpolationUtils::DistortionMapBlend(DistortionTable, InFocus, InZoom, Params);
	if (!Results.bValid || !Results.BlendedDistortionParams.IsSet())
	{
		// We were unable to generate blended distortion parameters for some reason. Fallback to default distortion,
		// while keeping blended image center and focal length
		
		// Initialize all distortion parameters with their default values
		LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(InterpolatedState.DistortionInfo.Parameters);
		
		// Setup handler state based on evaluated parameters. If none were found, no distortion will be returned
		InLensHandler->SetDistortionState(InterpolatedState);
		InLensHandler->SetCameraFilmback(CameraFilmback);

		InLensHandler->SetOverscanFactor(1.0f);

		//Draw displacement map associated with the new state
		InLensHandler->ProcessCurrentDistortion();
		return true;
	}

	InterpolatedState.DistortionInfo.Parameters = Results.BlendedDistortionParams->Parameters;
	
	//Sets final blended distortion state
	InLensHandler->SetDistortionState(InterpolatedState);
	InLensHandler->SetCameraFilmback(CameraFilmback);

	//Draw resulting undistortion displacement map for evaluation point
	LensFileRendering::DrawBlendedDisplacementMap(InLensHandler->GetUndistortionDisplacementMap()
		, Results.BlendingParams.GetValue()
		, UndistortionDisplacementMapHolders[0]
		, UndistortionDisplacementMapHolders[1]
		, UndistortionDisplacementMapHolders[2]
		, UndistortionDisplacementMapHolders[3]);

	//Draw resulting distortion displacement map for evaluation point
	LensFileRendering::DrawBlendedDisplacementMap(InLensHandler->GetDistortionDisplacementMap()
		, Results.BlendingParams.GetValue()
		, DistortionDisplacementMapHolders[0]
		, DistortionDisplacementMapHolders[1]
		, DistortionDisplacementMapHolders[2]
		, DistortionDisplacementMapHolders[3]);

	InLensHandler->SetOverscanFactor(Results.BlendedOverscan.Get(1.0f));
	
	return true;
}

bool ULensFile::EvaluateDistortionForSTMaps(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* InLensHandler) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateDistortionForSTMaps);

	using FBlendParams = LensInterpolationUtils::FDistortionMapBlendParams<FSTMapTable>;
	using FBlendResults = LensInterpolationUtils::FDistortionMapBlendResults;
	
	if(DerivedDataInFlightCount > 0)
	{
		UE_LOGF(LogCameraCalibrationCore, Verbose, "Can't evaluate LensFile '%ls' - %d data points still being computed. Clearing render target for no distortion", *GetName(), DerivedDataInFlightCount);
		SetupNoDistortionOutput(InLensHandler);
		return true;
	}
	
	if (((LensInfo.SensorDimensions.X + UE_DOUBLE_KINDA_SMALL_NUMBER) < InFilmback.X) || ((LensInfo.SensorDimensions.Y + UE_DOUBLE_KINDA_SMALL_NUMBER) < InFilmback.Y))
	{
		UE_LOG(LogCameraCalibrationCore, Verbose
			, TEXT("Can't evaluate LensFile '%s' - The filmback used to generate the calibrated ST Maps is smaller than" \
				"the filmback of the camera that distortion is being applied to. There is not enough distortion information available in the ST Maps.")
			, *GetName());
		SetupNoDistortionOutput(InLensHandler);
		return false;
	}

	FCameraFilmbackSettings CameraFilmback;
	CameraFilmback.SensorWidth = InFilmback.X;
	CameraFilmback.SensorHeight = InFilmback.Y;
	
	const FVector2D FxFyScale = FVector2D(InFilmback.X / LensInfo.SensorDimensions.X, InFilmback.Y / LensInfo.SensorDimensions.Y);
	
	//When dealing with STMaps, FxFy was not a calibrated value. We can interpolate our curve directly for desired point
	FLensDistortionState InterpolatedState;
	EvaluateImageCenterParameters(InFocus, InZoom, InterpolatedState.ImageCenter);
	EvaluateFocalLength(InFocus, InZoom, InterpolatedState.FocalLengthInfo);
	InterpolatedState.FocalLengthInfo.FxFy *= FxFyScale;

	FBlendParams Params;
	Params.bGenerateBlendingParams = true;
	Params.bGenerateDistortionMaps = true;
	Params.bGenerateBlendedOverscan = true;

	Params.GetOverscan = FBlendParams::FGetOverscan::CreateLambda(
		[this]
		(const FSTMapFocusPoint& FocusPoint, float Zoom, const FLensDistortionState& InDistortionState)
		{
			if (const FSTMapZoomPoint* ZoomPoint = FocusPoint.GetZoomPoint(Zoom))
			{
				return ComputeOverscan(ZoomPoint->DerivedDistortionData.DistortionData, InDistortionState.ImageCenter.PrincipalPoint);
			}

			return 1.0f;
		});
	
	FBlendResults Results = LensInterpolationUtils::DistortionMapBlend(STMapTable, InFocus, InZoom, Params);
	if (!Results.bValid || !Results.UndistortedMaps.IsSet() || !Results.DistortedMaps.IsSet())
	{
		UE_LOGF(LogCameraCalibrationCore, Verbose, "Can't evaluate LensFile '%ls' - No calibrated maps", *GetName());
		SetupNoDistortionOutput(InLensHandler);
		return true;
	}

	FDisplacementMapBlendingParams& BlendingParams = Results.BlendingParams.GetValue();
	BlendingParams.FxFyScale = FxFyScale;
	BlendingParams.PrincipalPoint = InterpolatedState.ImageCenter.PrincipalPoint;
	
	//Draw resulting undistortion displacement map for evaluation point
	LensFileRendering::DrawBlendedDisplacementMap(InLensHandler->GetUndistortionDisplacementMap()
		, BlendingParams
		, Results.UndistortedMaps.GetValue()[0]
		, Results.UndistortedMaps.GetValue()[1]
		, Results.UndistortedMaps.GetValue()[2]
		, Results.UndistortedMaps.GetValue()[3]);

	//Draw resulting distortion displacement map for evaluation point
	LensFileRendering::DrawBlendedDisplacementMap(InLensHandler->GetDistortionDisplacementMap()
		, BlendingParams
		, Results.DistortedMaps.GetValue()[0]
		, Results.DistortedMaps.GetValue()[1]
		, Results.DistortedMaps.GetValue()[2]
		, Results.DistortedMaps.GetValue()[3]);
	
	//Sets final blended distortion state
	InLensHandler->SetDistortionState(MoveTemp(InterpolatedState));
	InLensHandler->SetCameraFilmback(CameraFilmback);

	InLensHandler->SetOverscanFactor(Results.BlendedOverscan.Get(1.0f));
	
	return true;
}

bool ULensFile::EvaluateNodalPointOffset(float InFocus, float InZoom, FNodalPointOffset& OutEvaluatedValue) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULensFile::EvaluateNodalPointOffset);

	constexpr int32 NumParams = 6;
	TArray<float> BlendedParameters;
	if (LensInterpolationUtils::IndexedParameterBlend(NodalOffsetTable.FocusPoints, NodalOffsetTable.FocusCurves, InFocus, InZoom, NumParams, BlendedParameters))
	{
		ensure(BlendedParameters.Num() == NumParams);

		FVector Location;
		FRotator Rotation;
		
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Location[Index] = BlendedParameters[Index];
			Rotation.SetComponentForAxis(static_cast<EAxis::Type>(Index + 1), BlendedParameters[Index + 3]);
		}

		OutEvaluatedValue.LocationOffset = Location;
		OutEvaluatedValue.RotationOffset = FQuat(Rotation);
		return true;
	}
	
	return false;
}

bool ULensFile::HasFocusEncoderMapping() const
{
	return EncodersTable.Focus.GetNumKeys() > 0;
}

float ULensFile::EvaluateNormalizedFocus(float InNormalizedValue) const
{
	return EncodersTable.Focus.Eval(InNormalizedValue);
}

bool ULensFile::HasIrisEncoderMapping() const
{
	return EncodersTable.Iris.GetNumKeys() > 0;
}

float ULensFile::EvaluateNormalizedIris(float InNormalizedValue) const
{
	return EncodersTable.Iris.Eval(InNormalizedValue);
}

void ULensFile::OnDistortionDerivedDataJobCompleted(const FDerivedDistortionDataJobOutput& JobOutput)
{
	//Keep track of jobs being processed
	--DerivedDataInFlightCount;

	if(FSTMapFocusPoint* FocusPoint = STMapTable.GetFocusPoint(JobOutput.Focus))
	{
		if(FSTMapZoomPoint* ZoomPoint = FocusPoint->GetZoomPoint(JobOutput.Zoom))
		{
			if (JobOutput.Result == EDerivedDistortionDataResult::Success)
			{
				ZoomPoint->DerivedDistortionData.DistortionData.DistortedUVs = JobOutput.EdgePointsDistortedUVs;	
			}
			else
			{
				UE_LOGF(LogCameraCalibrationCore, Warning, "Could not derive distortion data for calibrated map point with Focus = '%0.2f' and Zoom = '%0.2f' on LensFile '%ls'", FocusPoint->Focus, ZoomPoint->Zoom, *GetName());
			}
		}
	}
}

void ULensFile::UpdateInputTolerance(const float NewTolerance)
{
	InputTolerance = NewTolerance;
}

TArray<FDistortionPointInfo> ULensFile::GetDistortionPoints() const
{
	return LensDataTableUtils::GetAllPointsInfo<FDistortionPointInfo>(DistortionTable);
}

TArray<FFocalLengthPointInfo> ULensFile::GetFocalLengthPoints() const
{
	return LensDataTableUtils::GetAllPointsInfo<FFocalLengthPointInfo>(FocalLengthTable);
}

TArray<FSTMapPointInfo> ULensFile::GetSTMapPoints() const
{
	return LensDataTableUtils::GetAllPointsInfo<FSTMapPointInfo>(STMapTable);
}

TArray<FImageCenterPointInfo> ULensFile::GetImageCenterPoints() const
{
	return LensDataTableUtils::GetAllPointsInfo<FImageCenterPointInfo>(ImageCenterTable);
}

TArray<FNodalOffsetPointInfo> ULensFile::GetNodalOffsetPoints() const
{
	return LensDataTableUtils::GetAllPointsInfo<FNodalOffsetPointInfo>(NodalOffsetTable);
}

bool ULensFile::GetDistortionPoint(const float InFocus, const float InZoom, FDistortionInfo& OutDistortionInfo) const
{
	return DistortionTable.GetPoint(InFocus, InZoom, OutDistortionInfo, InputTolerance);
}

bool ULensFile::GetFocalLengthPoint(const float InFocus, const float InZoom, FFocalLengthInfo& OutFocalLengthInfo) const
{
	return FocalLengthTable.GetPoint(InFocus, InZoom, OutFocalLengthInfo, InputTolerance);
}

bool ULensFile::GetImageCenterPoint(const float InFocus, const float InZoom, FImageCenterInfo& OutImageCenterInfo) const
{
	return ImageCenterTable.GetPoint(InFocus, InZoom, OutImageCenterInfo, InputTolerance);
}

bool ULensFile::GetNodalOffsetPoint(const float InFocus, const float InZoom, FNodalPointOffset& OutNodalPointOffset) const
{
	return NodalOffsetTable.GetPoint(InFocus, InZoom, OutNodalPointOffset, InputTolerance);
}

bool ULensFile::GetSTMapPoint(const float InFocus, const float InZoom, FSTMapInfo& OutSTMapInfo) const
{
	return STMapTable.GetPoint(InFocus, InZoom, OutSTMapInfo, InputTolerance);
}

void ULensFile::AddDistortionPoint(float NewFocus, float NewZoom, const FDistortionInfo& NewDistortionPoint, const FFocalLengthInfo& NewFocalLength)
{
	const bool bPointAdded = DistortionTable.AddPoint(NewFocus, NewZoom, NewDistortionPoint, InputTolerance, false);
	FocalLengthTable.AddPoint(NewFocus, NewZoom, NewFocalLength, InputTolerance, bPointAdded);
}

void ULensFile::AddFocalLengthPoint(float NewFocus, float NewZoom, const FFocalLengthInfo& NewFocalLength)
{
	FocalLengthTable.AddPoint(NewFocus, NewZoom, NewFocalLength, InputTolerance, false);
}

void ULensFile::AddImageCenterPoint(float NewFocus, float NewZoom, const FImageCenterInfo& NewImageCenterPoint)
{
	ImageCenterTable.AddPoint(NewFocus, NewZoom, NewImageCenterPoint, InputTolerance, false);
}

void ULensFile::AddNodalOffsetPoint(float NewFocus, float NewZoom, const FNodalPointOffset& NewNodalOffsetPoint)
{
	NodalOffsetTable.AddPoint(NewFocus, NewZoom, NewNodalOffsetPoint, InputTolerance, false);
}

void ULensFile::AddSTMapPoint(float NewFocus, float NewZoom, const FSTMapInfo& NewPoint)
{
	STMapTable.AddPoint(NewFocus, NewZoom, NewPoint, InputTolerance, false);
}

// Helper macros that fill in switch cases for the base lens tables for calling a function
#define FUNCTION_LENS_TABLE_CASE(DataCategoryName, TableName, FuncCall) case DataCategoryName: TableName.FuncCall; break;
#define SWITCH_FUNCTION_ON_BASE_LENS_TABLES(DataCategory, FuncCall) FUNCTION_LENS_TABLE_CASE(ELensDataCategory::Distortion, DistortionTable, FuncCall) \
	FUNCTION_LENS_TABLE_CASE(ELensDataCategory::ImageCenter, ImageCenterTable, FuncCall) \
	FUNCTION_LENS_TABLE_CASE(ELensDataCategory::Zoom, FocalLengthTable, FuncCall) \
	FUNCTION_LENS_TABLE_CASE(ELensDataCategory::STMap, STMapTable, FuncCall) \
	FUNCTION_LENS_TABLE_CASE(ELensDataCategory::NodalOffset, NodalOffsetTable, FuncCall) \

#define FUNCTION_RETVAL_LENS_TABLE_CASE(DataCategoryName, TableName, FuncCall) case DataCategoryName: return TableName.FuncCall;
#define SWITCH_FUNCTION_RETVAL_ON_BASE_LENS_TABLES(DataCategory, FuncCall) FUNCTION_RETVAL_LENS_TABLE_CASE(ELensDataCategory::Distortion, DistortionTable, FuncCall) \
	FUNCTION_RETVAL_LENS_TABLE_CASE(ELensDataCategory::ImageCenter, ImageCenterTable, FuncCall) \
	FUNCTION_RETVAL_LENS_TABLE_CASE(ELensDataCategory::Zoom, FocalLengthTable, FuncCall) \
	FUNCTION_RETVAL_LENS_TABLE_CASE(ELensDataCategory::STMap, STMapTable, FuncCall) \
	FUNCTION_RETVAL_LENS_TABLE_CASE(ELensDataCategory::NodalOffset, NodalOffsetTable, FuncCall) \

void ULensFile::RemoveFocusPoint(ELensDataCategory InDataCategory, float InFocus)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_ON_BASE_LENS_TABLES(InDataCategory, RemoveFocusPoint(InFocus))
		
		case ELensDataCategory::Focus:
		{
			EncodersTable.RemoveFocusPoint(InFocus);
			break;
		}
		case ELensDataCategory::Iris:
		{
			EncodersTable.RemoveIrisPoint(InFocus);
			break;
		}
		default:
		{}
	}
}

bool ULensFile::HasFocusPoint(ELensDataCategory InDataCategory, float InFocus) const
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_RETVAL_ON_BASE_LENS_TABLES(InDataCategory, HasFocusPoint(InFocus, InputTolerance))

		// Unsupported on encoder tables
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{
			return false;
		}
	}
}

void ULensFile::ChangeFocusPoint(ELensDataCategory InDataCategory, float InExistingFocus, float InNewFocus)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_ON_BASE_LENS_TABLES(InDataCategory, ChangeFocusPoint(InExistingFocus, InNewFocus, InputTolerance))

		// Changing focus points is unsupported on encoder tables
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{}
	}
}

void ULensFile::MergeFocusPoint(ELensDataCategory InDataCategory, float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_ON_BASE_LENS_TABLES(InDataCategory, MergeFocusPoint(InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance))

		// Merging focus points is unsupported on encoder tables
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{}
	}
}

void ULensFile::RemoveZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InZoom)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_ON_BASE_LENS_TABLES(InDataCategory, RemoveZoomPoint(InFocus, InZoom))

		// Encoder tables don't have zoom points
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{}
	}
}

bool ULensFile::HasZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InZoom)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_RETVAL_ON_BASE_LENS_TABLES(InDataCategory, HasZoomPoint(InFocus, InZoom, InputTolerance))

		// Encoder tables don't have zoom points
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{
			return false;
		}
	}
}

void ULensFile::ChangeZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InExistingZoom, float InNewZoom)
{
	switch(InDataCategory)
	{
		SWITCH_FUNCTION_ON_BASE_LENS_TABLES(InDataCategory, ChangeZoomPoint(InFocus, InExistingZoom, InNewZoom, InputTolerance))

		// Encoder tables don't have zoom points
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{}
	}
}

void ULensFile::ClearAll()
{
	EncodersTable.ClearAll();
	LensDataTableUtils::EmptyTable(DistortionTable);
	LensDataTableUtils::EmptyTable(FocalLengthTable);
	LensDataTableUtils::EmptyTable(STMapTable);
	LensDataTableUtils::EmptyTable(ImageCenterTable);
	LensDataTableUtils::EmptyTable(NodalOffsetTable);
}

void ULensFile::ClearData(ELensDataCategory InDataCategory)
{
	switch(InDataCategory)
	{
		case ELensDataCategory::Distortion:
		{
			LensDataTableUtils::EmptyTable(DistortionTable);
			break;
		}
		case ELensDataCategory::ImageCenter:
		{
			LensDataTableUtils::EmptyTable(ImageCenterTable);
			break;
		}
		case ELensDataCategory::Zoom:
		{
			LensDataTableUtils::EmptyTable(FocalLengthTable);
				break;
		}
		case ELensDataCategory::STMap:
		{
			LensDataTableUtils::EmptyTable(STMapTable);
				break;
		}
		case ELensDataCategory::NodalOffset:
		{
			LensDataTableUtils::EmptyTable(NodalOffsetTable);
			break;
		}
		case ELensDataCategory::Focus:
		{
			EncodersTable.Focus.Reset();
			break;
		}
		case ELensDataCategory::Iris:
		{
			EncodersTable.Iris.Reset();
			break;
		}
		default:
		{
		}
	}
}

bool ULensFile::HasSamples(ELensDataCategory InDataCategory) const
{
	return GetTotalPointNum(InDataCategory) > 0 ? true : false;
}

int32 ULensFile::GetTotalPointNum(ELensDataCategory InDataCategory) const
{
	switch(InDataCategory)
	{
		case ELensDataCategory::Distortion:
		{
			return DistortionTable.GetTotalPointNum();
		}
		case ELensDataCategory::ImageCenter:
		{
			return ImageCenterTable.GetTotalPointNum();
		}
		case ELensDataCategory::Zoom:
		{
			return FocalLengthTable.GetTotalPointNum();
		}
		case ELensDataCategory::STMap:
		{
			return STMapTable.GetTotalPointNum();
		}
		case ELensDataCategory::NodalOffset:
		{
			return NodalOffsetTable.GetTotalPointNum();
		}
		case ELensDataCategory::Focus:
		{
			return EncodersTable.GetNumFocusPoints();
		}
		case ELensDataCategory::Iris:
		{
			return EncodersTable.GetNumIrisPoints();
		}
		default:
		{
			return -1;
		}
	}
}

const FBaseLensTable* ULensFile::GetDataTable(ELensDataCategory InDataCategory) const
{
	switch(InDataCategory)
	{
		case ELensDataCategory::Distortion:
		{
			return &DistortionTable;
		}
		case ELensDataCategory::ImageCenter:
		{
			return &ImageCenterTable;
		}
		case ELensDataCategory::Zoom:
		{
			return &FocalLengthTable;
		}
		case ELensDataCategory::STMap:
		{
			return &STMapTable;
		}
		case ELensDataCategory::NodalOffset:
		{
			return &NodalOffsetTable;
		}
		case ELensDataCategory::Focus:
		case ELensDataCategory::Iris:
		default:
		{
			// No base table for now.
			return nullptr;
		}
	}
}

FBaseLensTable* ULensFile::GetDataTable(ELensDataCategory InDataCategory)
{
	return const_cast<FBaseLensTable*>(const_cast<const ULensFile*>(this)->GetDataTable(InDataCategory));
}

void ULensFile::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Enable ticking now that construction is complete and we're on the game thread.
		// Constructor uses ETickableTickType::Never because UObjects can be constructed on worker threads.
		SetTickableTickType(ETickableTickType::Conditional);

		const FIntPoint DisplacementMapResolution = GetDefault<UCameraCalibrationSettings>()->GetDisplacementMapResolution();
		CreateIntermediateDisplacementMaps(DisplacementMapResolution);
	}
	
	// Set a Lens file reference to all tables
	DistortionTable.LensFile =
		FocalLengthTable.LensFile = ImageCenterTable.LensFile =
		NodalOffsetTable.LensFile = STMapTable.LensFile = this;

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
}

void ULensFile::Tick(float DeltaTime)
{
	if (CalibratedMapProcessor)
	{
		CalibratedMapProcessor->Update();
	}

	UpdateDerivedData();
}

void ULensFile::UpdateDisplacementMapResolution(const FIntPoint NewDisplacementMapResolution)
{
	CreateIntermediateDisplacementMaps(NewDisplacementMapResolution);

	// Mark all points in the STMap table as dirty, so that they will update their derived data on the next tick
	if (DataMode == ELensDataMode::STMap)
	{
		for (FSTMapFocusPoint& FocusPoint : STMapTable.GetFocusPoints())
		{
			for (FSTMapZoomPoint& ZoomPoint : FocusPoint.ZoomPoints)
			{
				ZoomPoint.DerivedDistortionData.bIsDirty = true;
			}
		}
	}
}

TStatId ULensFile::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(ULensFile, STATGROUP_Tickables);
}

void ULensFile::UpdateDerivedData()
{
	if(DataMode == ELensDataMode::STMap)
	{
		for(FSTMapFocusPoint& FocusPoint : STMapTable.GetFocusPoints())
		{
			for (FSTMapZoomPoint& ZoomPoint : FocusPoint.ZoomPoints)
			{
				if (ZoomPoint.DerivedDistortionData.bIsDirty)
				{
					//Early exit if source map does not exist
					if (ZoomPoint.STMapInfo.DistortionMap == nullptr)
					{
						ZoomPoint.DerivedDistortionData.bIsDirty = false;
						continue;
					}

					//Early exit it the source map is not yet loaded (but leave it marked dirty so it tries again later)
					if (ZoomPoint.STMapInfo.DistortionMap->GetResource() == nullptr ||
						ZoomPoint.STMapInfo.DistortionMap->GetResource()->IsProxy())
					{
						continue;
					}

					const FIntPoint CurrentDisplacementMapResolution = GetDefault<UCameraCalibrationSettings>()->GetDisplacementMapResolution();

					//Create required undistortion texture for newly added points
					if ((ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap == nullptr)
						|| (ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap->SizeX != CurrentDisplacementMapResolution.X)
						|| (ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap->SizeY != CurrentDisplacementMapResolution.Y))

					{
						ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap = LensFileUtils::CreateDisplacementMapRenderTarget(this, CurrentDisplacementMapResolution);
					}

					//Create required distortion texture for newly added points
					if ((ZoomPoint.DerivedDistortionData.DistortionDisplacementMap == nullptr)
						|| (ZoomPoint.DerivedDistortionData.DistortionDisplacementMap->SizeX != CurrentDisplacementMapResolution.X)
						|| (ZoomPoint.DerivedDistortionData.DistortionDisplacementMap->SizeY != CurrentDisplacementMapResolution.Y))
					{
						ZoomPoint.DerivedDistortionData.DistortionDisplacementMap = LensFileUtils::CreateDisplacementMapRenderTarget(this, CurrentDisplacementMapResolution);
					}

					check(ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap);
					check(ZoomPoint.DerivedDistortionData.DistortionDisplacementMap);

					FDerivedDistortionDataJobArgs JobArgs;
					JobArgs.Focus = FocusPoint.Focus;
					JobArgs.Zoom = ZoomPoint.Zoom;
					JobArgs.Format = ZoomPoint.STMapInfo.MapFormat;
					JobArgs.SourceDistortionMap = ZoomPoint.STMapInfo.DistortionMap;
					JobArgs.OutputUndistortionDisplacementMap = ZoomPoint.DerivedDistortionData.UndistortionDisplacementMap;
					JobArgs.OutputDistortionDisplacementMap = ZoomPoint.DerivedDistortionData.DistortionDisplacementMap;
					JobArgs.JobCompletedCallback.BindUObject(this, &ULensFile::OnDistortionDerivedDataJobCompleted);
					if (CalibratedMapProcessor->PushDerivedDistortionDataJob(MoveTemp(JobArgs)))
					{
						++DerivedDataInFlightCount;
						ZoomPoint.DerivedDistortionData.bIsDirty = false;
					}
				}
			}
		
		}
	}
}

void ULensFile::CreateIntermediateDisplacementMaps(const FIntPoint DisplacementMapResolution)
{
	UndistortionDisplacementMapHolders.Reset(DisplacementMapHolderCount);
	DistortionDisplacementMapHolders.Reset(DisplacementMapHolderCount);
	for (int32 Index = 0; Index < DisplacementMapHolderCount; ++Index)
	{
		UTextureRenderTarget2D* NewUndistortionMap = LensFileUtils::CreateDisplacementMapRenderTarget(GetTransientPackage(), DisplacementMapResolution);
		UTextureRenderTarget2D* NewDistortionMap = LensFileUtils::CreateDisplacementMapRenderTarget(GetTransientPackage(), DisplacementMapResolution);
		UndistortionDisplacementMapHolders.Add(NewUndistortionMap);
		DistortionDisplacementMapHolders.Add(NewDistortionMap);
	}
}

ULensFile* FLensFilePicker::GetLensFile() const
{
	if(bUseDefaultLensFile)
	{
		UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
		return SubSystem->GetDefaultLensFile();
	}
	else
	{
		return LensFile;
	}
}

#if WITH_EDITOR
void ULensFile::BuildLensTableFocusCurves()
{
	FocalLengthTable.BuildFocusCurves();
	DistortionTable.BuildFocusCurves();
	ImageCenterTable.BuildFocusCurves();
	STMapTable.BuildFocusCurves();
	NodalOffsetTable.BuildFocusCurves();
}
#endif // WITH_EDITOR
