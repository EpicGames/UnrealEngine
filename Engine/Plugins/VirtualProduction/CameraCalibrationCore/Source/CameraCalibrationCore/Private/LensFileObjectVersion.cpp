// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensFileObjectVersion.h"

#include "UObject/DevObjectVersion.h"

const FGuid FLensFileObjectVersion::GUID(0x8652A554, 0x966A466C, 0x9FD71C6D, 0xD61B1ADB);
static FDevVersionRegistration GRegisterLensFileObjectVersion(FLensFileObjectVersion::GUID, FLensFileObjectVersion::LatestVersion, TEXT("LensFileVersion"));