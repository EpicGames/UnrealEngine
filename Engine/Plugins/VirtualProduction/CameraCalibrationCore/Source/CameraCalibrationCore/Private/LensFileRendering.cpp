// Copyright Epic Games, Inc. All Rights Reserved.


#include "LensFileRendering.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Misc/MemStack.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterStruct.h"
#include "ScreenPass.h"
#include "TextureResource.h"


/** A pixel shader that blends displacement maps together. */
class FDisplacementMapBlendPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDisplacementMapBlendPS);
	SHADER_USE_PARAMETER_STRUCT(FDisplacementMapBlendPS, FGlobalShader);

	class FBlendType : SHADER_PERMUTATION_INT("BLEND_TYPE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FBlendType>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )

		SHADER_PARAMETER(float, EvalFocus)
		SHADER_PARAMETER(float, EvalZoom)
		SHADER_PARAMETER_ARRAY(FVector4f, PatchCorners, [4])
		SHADER_PARAMETER(FVector2f, FxFyScale)
		SHADER_PARAMETER(FVector2f, PrincipalPoint)
		SHADER_PARAMETER(FIntPoint, OutputTextureExtent)
        SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTextureOne)
        SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTextureTwo)
        SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTextureThree)
        SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTextureFour)
        SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)

        RENDER_TARGET_BINDING_SLOTS()
    END_SHADER_PARAMETER_STRUCT()

	// Called by the engine to determine which permutations to compile for this shader
    static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

IMPLEMENT_GLOBAL_SHADER(FDisplacementMapBlendPS, "/Plugin/CameraCalibrationCore/Private/DisplacementMapBlending.usf", "BlendPS", SF_Pixel);


namespace LensFileRendering
{
	void ClearDisplacementMap(UTextureRenderTarget2D* OutRenderTarget)
	{
		if (OutRenderTarget != nullptr)
		{
			const FTextureRenderTargetResource* const DestinationTextureResource = OutRenderTarget->GameThread_GetRenderTargetResource();

			ENQUEUE_RENDER_COMMAND(LensFileRendering_ClearDisplacementMap)(
			[DestinationTextureResource](FRHICommandListImmediate& RHICmdList)
			{
				FRDGBuilder GraphBuilder(RHICmdList);

				const FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTextureResource->TextureRHI, TEXT("OutputDisplacement")));

				FLinearColor NoDistortionColor(0.0f, 0.0f, 0.0f, 0.0f);
				AddClearRenderTargetPass(GraphBuilder, OutputTexture, NoDistortionColor);

				GraphBuilder.Execute();
			});
		}
	}	
	
	bool DrawBlendedDisplacementMap(UTextureRenderTarget2D* OutRenderTarget
	, const FDisplacementMapBlendingParams& BlendParams
	, UTextureRenderTarget2D* SourceTextureOne
	, UTextureRenderTarget2D* SourceTextureTwo
	, UTextureRenderTarget2D* SourceTextureThree
	, UTextureRenderTarget2D* SourceTextureFour)
{
	// At minimum, there must be one valid source texture and a valid output
	if (SourceTextureOne == nullptr || OutRenderTarget == nullptr)
	{
		return false;
	}
		
	// Verify that any additional required source textures are valid for the input blend type before proceeding
	switch (BlendParams.BlendType)
	{
	case EDisplacementMapBlendType::TwoFocusOneZoom:
	case EDisplacementMapBlendType::OneFocusTwoZoom:
		{
			if (SourceTextureTwo == nullptr)
			{
				return false;
			}
			break;
		}
	case EDisplacementMapBlendType::TwoFocusTwoZoom:
		{
			if (SourceTextureTwo == nullptr || SourceTextureThree == nullptr || SourceTextureFour == nullptr)
			{
				return false;
			}
			break;
		}
	default:
		{
			break;
		}
	}


	const FTextureRenderTargetResource* const SourceTextureOneResource = SourceTextureOne->GameThread_GetRenderTargetResource();
	const FTextureRenderTargetResource* const SourceTextureTwoResource = SourceTextureTwo ? SourceTextureTwo->GameThread_GetRenderTargetResource() : nullptr;
	const FTextureRenderTargetResource* const SourceTextureThreeResource = SourceTextureThree? SourceTextureThree->GameThread_GetRenderTargetResource() : nullptr;
	const FTextureRenderTargetResource* const SourceTextureFourResource = SourceTextureFour ? SourceTextureFour->GameThread_GetRenderTargetResource() : nullptr;
	const FTextureRenderTargetResource* const DestinationTextureResource = OutRenderTarget->GameThread_GetRenderTargetResource();

	ENQUEUE_RENDER_COMMAND(LensFileRendering_DrawBlendedDisplacementMap)(
		[SourceTextureOneResource, SourceTextureTwoResource, SourceTextureThreeResource, SourceTextureFourResource, BlendParams = BlendParams, DestinationTextureResource](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList);

			FDisplacementMapBlendPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDisplacementMapBlendPS::FParameters>();

			//Setup always used parameters
			FRDGTextureRef TextureOne = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureOneResource->TextureRHI, TEXT("DisplacementMapOne")));
			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTextureResource->TextureRHI, TEXT("OutputDisplacement")));
			PassParameters->OutputTextureExtent = OutputTexture->Desc.Extent;
			PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(); 
			PassParameters->SourceTextureOne = TextureOne;
			PassParameters->FxFyScale = FVector2f(BlendParams.FxFyScale);	// LWC_TODO: Precision loss
			PassParameters->PrincipalPoint = FVector2f(BlendParams.PrincipalPoint);	// LWC_TODO: Precision loss

			//Setup parameters based on blending type
			FDisplacementMapBlendPS::FPermutationDomain PermutationVector;
			switch(BlendParams.BlendType)
			{
			case EDisplacementMapBlendType::TwoFocusOneZoom:
				{
					PermutationVector.Set<FDisplacementMapBlendPS::FBlendType>(1);
					check(SourceTextureTwoResource);
					const FRDGTextureRef TextureTwo = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureTwoResource->TextureRHI, TEXT("DisplacementMapTwo")));
					PassParameters->EvalFocus = BlendParams.EvalFocus;
					PassParameters->EvalZoom = BlendParams.EvalZoom;
					PassParameters->PatchCorners[0] = BlendParams.PatchCorners[0].ToVector();
					PassParameters->PatchCorners[1] = BlendParams.PatchCorners[1].ToVector();
					PassParameters->PatchCorners[2] = FVector4f::Zero();
					PassParameters->PatchCorners[3] = FVector4f::Zero();
					PassParameters->SourceTextureTwo = TextureTwo;
					break;
				}
			case EDisplacementMapBlendType::OneFocusTwoZoom:
				{
					PermutationVector.Set<FDisplacementMapBlendPS::FBlendType>(2);
					check(SourceTextureTwoResource);
					const FRDGTextureRef TextureTwo = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureTwoResource->TextureRHI, TEXT("DisplacementMapTwo")));
					PassParameters->EvalFocus = BlendParams.EvalFocus;
					PassParameters->EvalZoom = BlendParams.EvalZoom;
					PassParameters->PatchCorners[0] = BlendParams.PatchCorners[0].ToVector();
					PassParameters->PatchCorners[1] = BlendParams.PatchCorners[1].ToVector();
					PassParameters->PatchCorners[2] = FVector4f::Zero();
					PassParameters->PatchCorners[3] = FVector4f::Zero();
					PassParameters->SourceTextureTwo = TextureTwo;
					break;
				}
			case EDisplacementMapBlendType::TwoFocusTwoZoom:
				{
					PermutationVector.Set<FDisplacementMapBlendPS::FBlendType>(3);
					check(SourceTextureTwoResource && SourceTextureThreeResource && SourceTextureFourResource);
					const FRDGTextureRef TextureTwo = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureTwoResource->TextureRHI, TEXT("DisplacementMapTwo")));
					const FRDGTextureRef TextureThree = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureThreeResource->TextureRHI, TEXT("DisplacementMapThree")));
					const FRDGTextureRef TextureFour = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureFourResource->TextureRHI, TEXT("DisplacementMapFour")));
					PassParameters->EvalFocus = BlendParams.EvalFocus;
					PassParameters->EvalZoom = BlendParams.EvalZoom;
					PassParameters->PatchCorners[0] = BlendParams.PatchCorners[0].ToVector();
					PassParameters->PatchCorners[1] = BlendParams.PatchCorners[1].ToVector();
					PassParameters->PatchCorners[2] = BlendParams.PatchCorners[2].ToVector();
					PassParameters->PatchCorners[3] = BlendParams.PatchCorners[3].ToVector();
					PassParameters->SourceTextureTwo = TextureTwo;
					PassParameters->SourceTextureThree = TextureThree;
					PassParameters->SourceTextureFour = TextureFour;
						
					break;
				}
			case EDisplacementMapBlendType::OneFocusOneZoom:
			default:
				{
					PermutationVector.Set<FDisplacementMapBlendPS::FBlendType>(0);
					break;
				}
			}

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			const TShaderMapRef<FDisplacementMapBlendPS> PixelShader(GlobalShaderMap, PermutationVector);

			FScreenPassRenderTarget SceneColorRenderTarget(OutputTexture, ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets[0] = SceneColorRenderTarget.GetRenderTargetBinding();

			FPixelShaderUtils::AddFullscreenPass(
						GraphBuilder
						, GlobalShaderMap
						, RDG_EVENT_NAME("BlendingLensDisplacementMap")
						, PixelShader
						, PassParameters
						, FIntRect(0, 0, OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y));            	

			GraphBuilder.Execute();
		});
	
	return true;
}
}

