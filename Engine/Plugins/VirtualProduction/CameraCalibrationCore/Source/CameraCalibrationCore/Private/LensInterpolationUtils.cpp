// Copyright Epic Games, Inc. All Rights Reserved.


#include "LensInterpolationUtils.h"

#include "LensData.h"
#include "Curves/CurveEvaluation.h"
#include "Curves/RichCurve.h"


//Property interpolation utils largely inspired from livelink interp code
namespace LensInterpolationUtils
{
	void InterpolateProperty(FProperty* Property, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData);

	template<typename Type>
	void Interpolate(const FStructProperty* StructProperty, float InBlendWeight, const void* DataA, const void* DataB, void* DataResult)
	{
		const Type* ValuePtrA = StructProperty->ContainerPtrToValuePtr<Type>(DataA);
		const Type* ValuePtrB = StructProperty->ContainerPtrToValuePtr<Type>(DataB);
		Type* ValuePtrResult = StructProperty->ContainerPtrToValuePtr<Type>(DataResult);

		Type ValueResult = BlendValue(InBlendWeight, *ValuePtrA, *ValuePtrB);
		StructProperty->CopySingleValue(ValuePtrResult, &ValueResult);
	}

	void Interpolate(const UStruct* InStruct, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData)
	{
		for (TFieldIterator<FProperty> Itt(InStruct); Itt; ++Itt)
		{
			FProperty* Property = *Itt;

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				//ArrayProps have an ArrayDim of 1 but just to be sure...
				for (int32 DimIndex = 0; DimIndex < ArrayProperty->ArrayDim; ++DimIndex)
				{
					const void* Data0 = ArrayProperty->ContainerPtrToValuePtr<const void>(InFrameDataA, DimIndex);
					const void* Data1 = ArrayProperty->ContainerPtrToValuePtr<const void>(InFrameDataB, DimIndex);
					void* DataResult = ArrayProperty->ContainerPtrToValuePtr<void>(OutFrameData, DimIndex);

					FScriptArrayHelper ArrayHelperA(ArrayProperty, Data0);
					FScriptArrayHelper ArrayHelperB(ArrayProperty, Data1);
					FScriptArrayHelper ArrayHelperResult(ArrayProperty, DataResult);

					const int32 MinValue = FMath::Min(ArrayHelperA.Num(), ArrayHelperB.Num());
					ArrayHelperResult.Resize(MinValue);

					for (int32 ArrayIndex = 0; ArrayIndex < MinValue; ++ArrayIndex)
					{
						InterpolateProperty(ArrayProperty->Inner, InBlendWeight, ArrayHelperA.GetRawPtr(ArrayIndex), ArrayHelperB.GetRawPtr(ArrayIndex), ArrayHelperResult.GetRawPtr(ArrayIndex));
					}
				}
			}
			else if (Property->ArrayDim > 1)
			{
				for (int32 DimIndex = 0; DimIndex < Property->ArrayDim; ++DimIndex)
				{
					const void* Data0 = Property->ContainerPtrToValuePtr<const void>(InFrameDataA, DimIndex);
					const void* Data1 = Property->ContainerPtrToValuePtr<const void>(InFrameDataB, DimIndex);
					void* DataResult = Property->ContainerPtrToValuePtr<void>(OutFrameData, DimIndex);

					InterpolateProperty(Property, InBlendWeight, Data0, Data1, DataResult);
				}
			}
			else
			{
				InterpolateProperty(Property, InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
			}
		}
	}

	void InterpolateProperty(FProperty* Property, float InBlendWeight, const void* InDataA, const void* InDataB, void* OutData)
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct->GetFName() == NAME_Vector)
			{
				Interpolate<FVector>(StructProperty, InBlendWeight, InDataA, InDataB, OutData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Vector2D)
			{
				Interpolate<FVector2D>(StructProperty, InBlendWeight, InDataA, InDataB, OutData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Vector4)
			{
				Interpolate<FVector4>(StructProperty, InBlendWeight, InDataA, InDataB, OutData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Rotator)
			{
				Interpolate<FRotator>(StructProperty, InBlendWeight, InDataA, InDataB, OutData);
			}
			else if (StructProperty->Struct->GetFName() == NAME_Quat)
			{
				Interpolate<FQuat>(StructProperty, InBlendWeight, InDataA, InDataB, OutData);
			}
			else
			{
				const void* Data0 = StructProperty->ContainerPtrToValuePtr<const void>(InDataA);
				const void* Data1 = StructProperty->ContainerPtrToValuePtr<const void>(InDataB);
				void* DataResult = StructProperty->ContainerPtrToValuePtr<void>(OutData);
				Interpolate(StructProperty->Struct, InBlendWeight, Data0, Data1, DataResult);
			}
		}
		else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
		{
			if (NumericProperty->IsFloatingPoint())
			{
				const void* Data0 = NumericProperty->ContainerPtrToValuePtr<const void>(InDataA);
				const double Value0 = NumericProperty->GetFloatingPointPropertyValue(Data0);
				const void* Data1 = NumericProperty->ContainerPtrToValuePtr<const void>(InDataB);
				const double Value1 = NumericProperty->GetFloatingPointPropertyValue(Data1);

				const double ValueResult = BlendValue(InBlendWeight, Value0, Value1);

				void* DataResult = NumericProperty->ContainerPtrToValuePtr<void>(OutData);
				NumericProperty->SetFloatingPointPropertyValue(DataResult, ValueResult);
			}
			else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
			{
				const void* Data0 = NumericProperty->ContainerPtrToValuePtr<const void>(InDataA);
				const int64 Value0 = NumericProperty->GetSignedIntPropertyValue(Data0);
				const void* Data1 = NumericProperty->ContainerPtrToValuePtr<const void>(InDataB);
				const int64 Value1 = NumericProperty->GetSignedIntPropertyValue(Data1);

				const int64 ValueResult = BlendValue(InBlendWeight, Value0, Value1);

				void* DataResult = NumericProperty->ContainerPtrToValuePtr<void>(OutData);
				NumericProperty->SetIntPropertyValue(DataResult, ValueResult);
			}
		}
	}

	float GetBlendFactor(float InValue, float ValueA, float ValueB)
	{
		//Keep input in range
		InValue = FMath::Clamp(InValue, ValueA, ValueB);

		const float Divider = ValueB - ValueA;
		if (!FMath::IsNearlyZero(Divider))
		{
			return (InValue - ValueA) / Divider;
		}
		else
		{
			return 1.0f;
		}
	}

	float FTangentBezierCurve::Eval(float InX)
	{
		if (FMath::IsNearlyEqual(X0, X1))
		{
			return Y0;
		}

		constexpr float OneThird = 1.0f / 3.0f;

		const float DeltaX = X1 - X0;
		const float DeltaY = Y1 - Y0;
		const float Alpha = (InX - X0) / DeltaX;
		const float TangentScale = DeltaY / DeltaX;
		
		const float P0 = Y0;
		const float P3 = Y1;
		const float P1 = P0 + (Tangent0 * TangentScale * DeltaX * OneThird);
		const float P2 = P3 - (Tangent1 * TangentScale * DeltaX * OneThird);
		return UE::Curves::BezierInterp(P0, P1, P2, P3, Alpha);
	}

	float FTangentBezierCurve::EvalTangent(float InX)
	{
		if (FMath::IsNearlyEqual(X0, X1))
		{
			return Tangent0;
		}

		if (FMath::IsNearlyEqual(Y0, Y1))
		{
			return 0.0f;
		}

		const float DeltaX = X1 - X0;
		const float DeltaY = Y1 - Y0;
		const float Alpha = (InX - X0) / DeltaX;

		const float T0 = Tangent0 * DeltaY;
		const float T2 = (3.0f - (Tangent1 + Tangent0)) * DeltaY;
		const float T3 = Tangent1 * DeltaY;
		
		return FMath::Lerp(FMath::Lerp(T0, T2, Alpha), FMath::Lerp(T2, T3, Alpha), Alpha) / DeltaY;
	}
	
	TArray<float> FTangentBezierCurve::MultiEval(float InX, const TArray<float>& InY0s, const TArray<float>& InY1s)
	{
		check(InY0s.Num() == InY1s.Num());
		
		TArray<float> Results;
		Results.AddZeroed(InY0s.Num());

		for (int32 Index = 0; Index < InY0s.Num(); ++Index)
		{
			Y0 = InY0s[Index];
			Y1 = InY1s[Index];

			Results[Index] = Eval(InX);
		}

		return Results;
	}

	float FCoonsPatch::Blend(const FVector2D& InPoint)
	{
		float Alpha = 0.0;
		if (!FMath::IsNearlyEqual(X0, X1))
		{
			Alpha = (InPoint.X - X0) / (X1 - X0);
		}
		
		float Beta = 0.0;
		if (!FMath::IsNearlyEqual(Y0, Y1))
		{
			Beta = (InPoint.Y - Y0) / (Y1 - Y0);
		}
		
		// In degenerate cases (such as only having 3 defined corners), the four curves' corners may not match.
		// Simply use the average of the two possible values in that case; while this doesn't match the strict
		// definition for a Coons patch, it allows the patch to give a somewhat usable value for such edge cases
		const float P00 = X0Curve.Eval(X0) + Y0Curve.Eval(Y0);
		const float P01 = X0Curve.Eval(X1) + Y1Curve.Eval(Y0);
		const float P10 = X1Curve.Eval(X0) + Y0Curve.Eval(Y1);
		const float P11 = X1Curve.Eval(X1) + Y1Curve.Eval(Y1);
			
		const float Lx = FMath::Lerp(X0Curve.Eval(InPoint.X), X1Curve.Eval(InPoint.X), Beta);
		const float Ly = FMath::Lerp(Y0Curve.Eval(InPoint.Y), Y1Curve.Eval(InPoint.Y), Alpha);
		const float B = FMath::BiLerp(P00, P01, P10, P11, Alpha, Beta);
		
		return Lx + Ly - 0.5 * B;
	}

	float FTangentBezierCoonsPatch::Blend(const FVector2D& InPoint)
	{
		float Alpha = 0.0f;
		if (!FMath::IsNearlyEqual(YCurves[0].Coordinate, YCurves[1].Coordinate))
		{
			Alpha = (InPoint.X - YCurves[0].Coordinate) / (YCurves[1].Coordinate - YCurves[0].Coordinate);
		}

		float Beta = 0.0f;
		if (!FMath::IsNearlyEqual(XCurves[0].Coordinate, XCurves[1].Coordinate))
		{
			Beta = (InPoint.Y - XCurves[0].Coordinate) / (XCurves[1].Coordinate - XCurves[0].Coordinate);
		}

		const float Lx = FMath::Lerp(XCurves[0].TangentCurve.Eval(InPoint.X), XCurves[0].TangentCurve.Eval(InPoint.X), Beta);
		const float Ly = FMath::Lerp(YCurves[0].TangentCurve.Eval(InPoint.Y), YCurves[0].TangentCurve.Eval(InPoint.Y), Alpha);

		float PatchCorners[4];
		for (int32 Index = 0; Index < 4; ++Index)
		{
			const int32 XIndex = Index / 2;
			const int32 YIndex = FMath::WrapExclusive(Index + 1, 0, 4) / 2;

			PatchCorners[Index] = 0.5f * (XCurves[XIndex].TangentCurve.Eval(YCurves[YIndex].Coordinate) + YCurves[YIndex].TangentCurve.Eval(XCurves[XIndex].Coordinate));
		}

		const float B = FMath::BiLerp(PatchCorners[0], PatchCorners[1], PatchCorners[3], PatchCorners[2], Alpha, Beta);
		return Lx + Ly - B;
	}

	FTangentBezierCurve GetTangentCurve(const FRichCurve& Curve, float Time, float MinTime, float MaxTime)
	{
		FTangentBezierCurve OutCurve(MinTime, MaxTime, 0.0, 0.0, 0.0, 0.0);
		
		const int32 NumKeys = Curve.GetNumKeys();
		if (NumKeys == 0)
		{
			return OutCurve;
		}

		if (Time < Curve.Keys[0].Time)
		{
			OutCurve.X1 = Curve.Keys[0].Time;
			return OutCurve;
		}
		if (Time >= Curve.Keys[NumKeys - 1].Time)
		{
			OutCurve.X0 = Curve.Keys[NumKeys - 1].Time;
			return OutCurve;
		}

		// Similar to RichCurve.Eval(), perform a binary search on the curve's keys to find the bounding points around Time
		int32 Start = 0;
		int32 End = NumKeys - 1;

		while (Start < End)
		{
			const int32 TestPos = Start + (End-Start) / 2;
			if (Time >= Curve.Keys[TestPos].Time)
			{
				Start = TestPos + 1;
			}
			else
			{
				End = TestPos;
			}
		}

		const FRichCurveKey& Prev = Curve.Keys[Start - 1];
		const FRichCurveKey& Next = Curve.Keys[Start];

		OutCurve.X0 = Prev.Time;
		OutCurve.X1 = Next.Time;
		
		if (Prev.InterpMode == ERichCurveInterpMode::RCIM_Cubic)
		{
			OutCurve.Tangent0 = Prev.LeaveTangent;
			OutCurve.Tangent1 = Next.ArriveTangent;
		}
		else
		{
			float Slope = (Next.Value - Prev.Value) / (Next.Time - Prev.Time);
			OutCurve.Tangent0 = Slope;
			OutCurve.Tangent1 = Slope;
		}

		return OutCurve;
	}

	template <>
	TOptional<FDistortionInfo> GetDistortionInfo<FDistortionTable>(const FDistortionFocusPoint& FocusPoint, float Zoom)
	{
		FDistortionInfo DistortionInfo;
		if (FocusPoint.GetPoint(Zoom, DistortionInfo))
		{
			return DistortionInfo;
		}

		return TOptional<FDistortionInfo>();
	}

	template <>
	void GetDisplacementMaps<FSTMapTable>(const FSTMapFocusPoint& FocusPoint, float Zoom, UTextureRenderTarget2D*& OutUndistortedMap, UTextureRenderTarget2D*& OutDistortedMap)
	{
		if (const FSTMapZoomPoint* ZoomPoint = FocusPoint.GetZoomPoint(Zoom))
		{
			OutUndistortedMap = ZoomPoint->DerivedDistortionData.UndistortionDisplacementMap;
			OutDistortedMap = ZoomPoint->DerivedDistortionData.DistortionDisplacementMap;
		}
	}

	template<>
	void FDistortionMapBlendParams<FDistortionTable>::InitializeResults(float InFocus, float InZoom, FDistortionMapBlendResults& OutResults) const
	{
		if (bGenerateBlendingParams)
		{
			OutResults.bValid = true;

			OutResults.BlendingParams = FDisplacementMapBlendingParams();
			OutResults.BlendingParams->EvalFocus = InFocus;
			OutResults.BlendingParams->EvalZoom = InZoom;
		}

		if (bGenerateBlendedDistortionParams)
		{
			OutResults.bValid = true;
			
			OutResults.BlendedDistortionParams = FDistortionInfo();
			OutResults.BlendedDistortionParams->Parameters = DefaultDistortionParams;
		}

		if (bGenerateDistortionMaps)
		{
			if (ProcessDisplacementMaps.IsBound() && UndistortedMaps.Num() == 4 && DistortedMaps.Num() == 4)
			{
				OutResults.bValid = true;
				
				OutResults.UndistortedMaps = UndistortedMaps;
				OutResults.DistortedMaps = DistortedMaps;
			}
		}

		if (bGenerateBlendedOverscan && GetOverscan.IsBound())
		{
			OutResults.bValid = true;
			
			OutResults.BlendedOverscan = 1.0f;
		}
	}

	template<>
	void FDistortionMapBlendParams<FSTMapTable>::InitializeResults(float InFocus, float InZoom, FDistortionMapBlendResults& OutResults) const
	{
		if (bGenerateBlendingParams)
		{
			OutResults.bValid = true;
			
			OutResults.BlendingParams = FDisplacementMapBlendingParams();
			OutResults.BlendingParams->EvalFocus = InFocus;
			OutResults.BlendingParams->EvalZoom = InZoom;
		}

		if (bGenerateDistortionMaps)
		{
			OutResults.bValid = true;
			
			OutResults.UndistortedMaps = TArray<UTextureRenderTarget2D*>();
			OutResults.DistortedMaps = TArray<UTextureRenderTarget2D*>();
			OutResults.UndistortedMaps->AddZeroed(4);
			OutResults.DistortedMaps->AddZeroed(4);
		}

		if (bGenerateBlendedOverscan && GetOverscan.IsBound())
		{
			OutResults.bValid = true;
			
			OutResults.BlendedOverscan = 1.0f;
		}
	}
}


