// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Curves/RichCurve.h"
#include "LensDistortionSceneViewExtension.h"
#include "LensFileRendering.h"
#include "Math/Vector2D.h"
#include "Tables/DistortionParametersTable.h"
#include "Tables/LensTableUtils.h"
#include "Tables/STMapTable.h"


struct FRichCurve;

namespace LensInterpolationUtils
{
	template<typename Type>
	Type BlendValue(float InBlendWeight, const Type& A, const Type& B)
	{
		return FMath::Lerp(A, B, InBlendWeight);
	}
	
	void Interpolate(const UStruct* InStruct, float InBlendWeight, const void* InFrameDataA, const void* InFrameDataB, void* OutFrameData);

	template<typename Type>
	void Interpolate(float InBlendWeight, const Type* InFrameDataA, const Type* InFrameDataB, Type* OutFrameData)
	{
		Interpolate(Type::StaticStruct(), InBlendWeight, InFrameDataA, InFrameDataB, OutFrameData);
	}
	
	float GetBlendFactor(float InValue, float ValueA, float ValueB);

	/** A cubic bezier curve constructed from two points and their tangents */
	struct FTangentBezierCurve
	{
		FTangentBezierCurve(float InX0, float InX1, float InY0, float InY1, float InTangent0, float InTangent1)
			: X0(InX0)
			, X1(InX1)
			, Y0(InY0)
			, Y1(InY1)
			, Tangent0(InTangent0)
			, Tangent1(InTangent1)

		{}

		/** Evaluates the Bezier curve at the specified x value */
		float Eval(float InX);

		/** Evaluates the tangent of the Bezier curve at the specified x value */
		float EvalTangent(float InX);
		
		/** Evaluates the Bezier curve for each of the provided y values at the specified x value */
		TArray<float> MultiEval(float InX, const TArray<float>& InY0s, const TArray<float>& InY1s);
		
		/** Minimum x value of the curve */
		float X0;

		/** Maximum x value of the curve */
		float X1;

		/** Y value of the curve at x0 */
		float Y0;

		/** Y value of the curve at x1 */
		float Y1;

		/** Tangent of the curve at x0 */
		float Tangent0;

		/** Tangent of the curve at x1 */
		float Tangent1;
	};
	
	/**
	 * A Coons patch (https://en.wikipedia.org/wiki/Coons_patch) for blending four boundary curves together
	 * into a surface patch. Takes in four rich curves for each side, two for x-axis and two for y-axis, as well as
	 * min and max values for x and y from which the corner points of the patch are formed.
	 */
	struct FCoonsPatch
	{
		FCoonsPatch(const FRichCurve& X0Curve, const FRichCurve& X1Curve, const FRichCurve& Y0Curve, const FRichCurve& Y1Curve, float X0, float X1, float Y0, float Y1)
			: X0Curve(X0Curve)
			, X1Curve(X1Curve)
			, Y0Curve(Y0Curve)
			, Y1Curve(Y1Curve)
			, X0(X0)
			, X1(X1)
			, Y0(Y0)
			, Y1(Y1)
		{ }
		
		/** Computes the value of the patch at the specified point by blending between the four edge curves */
		float Blend(const FVector2D& InPoint);
		
		/** Min x-axis curve of the patch */
		const FRichCurve& X0Curve;

		/** Max x-axis curve of the patch */
		const FRichCurve& X1Curve;

		/** Min y-axis curve of the patch */
		const FRichCurve& Y0Curve;

		/** Max y-axis curve of the patch */
		const FRichCurve& Y1Curve;

		/** Minimum x value of the patch */
		float X0;

		/** Maximum x value of the patch */
		float X1;

		/** Minimum y value of the patch */
		float Y0;

		/** Maximum y value of the patch */
		float Y1;
	};

	/**
	 * A version of the Coons patch that takes in four tangent Bezier curves, and performs
	 * a Bezier interpolation to create the four edge curves from the points and their tangents.
	 * Works in cases where the curves' endpoints do not match, in which case the four patch corners are
	 * computed by interpolation and averaged before the final Coons patch interpolation
	 */
	struct FTangentBezierCoonsPatch
	{
		struct FPatchCurve
		{
			/** The tangent curve along the primary axis of the patch edge */
			FTangentBezierCurve TangentCurve;

			/** The coordinate along the secondary axis that the curve is at */
			float Coordinate;

			FPatchCurve(FTangentBezierCurve InTangentCurve, float InCoordinate)
				: TangentCurve(InTangentCurve)
				, Coordinate(InCoordinate)
			{}
		};
		
		FTangentBezierCoonsPatch(const FPatchCurve& X0Curve, const FPatchCurve& X1Curve, const FPatchCurve& Y0Curve, const FPatchCurve& Y1Curve)
			: XCurves { X0Curve, X1Curve }
			, YCurves { Y0Curve, Y1Curve }
		{}
		
		/** Computes the value of the patch at the specified point by blending between the four edge curves */
		float Blend(const FVector2D& InPoint);

		/** The tangent curves along the x axes edges of the patch */
		FPatchCurve XCurves[2];

		/** The tangent curves along the y axes edges of the patch */
		FPatchCurve YCurves[2];
	};

	/**
	 * Constructs an appropriate tangent curve by seeking the nearest defined points that the specified Time is between
	 * @param Curve The curve to find defined points from
	 * @param Time The time value to find bounding points for
	 * @param MinTime If no defined point can be found less than Time, use this time value as the minimum
	 * @param MaxTime If no defined point can be found greater than Time, use this time value as the maximum
	 * @return A tangent Bezier curve that can be interpolated on
	 */
	FTangentBezierCurve GetTangentCurve(const FRichCurve& Curve, float Time, float MinTime, float MaxTime);
	
	/** Performs a Coons patch blend on an indexed list of parameters where each parameter has its own set of curves, and outputs all blended parameters */
	template<typename FocusPointType, typename FocusCurveType>
	bool IndexedParameterBlend(const TArray<FocusPointType>& FocusPoints, const TArray<FocusCurveType>& FocusCurves, float InFocus, float InZoom, int32 NumParameters, TArray<float>& OutBlendedParameters)
	{
		if (FocusPoints.Num() <= 0)
		{
			return false;
		}
		
		const LensDataTableUtils::FPointNeighbors PointNeighbors = LensDataTableUtils::FindFocusPoints(InFocus, TConstArrayView<FocusPointType>(FocusPoints));
		const LensDataTableUtils::FPointNeighbors CurveNeighbors = LensDataTableUtils::FindFocusCurves(InZoom, TConstArrayView<FocusCurveType>(FocusCurves));
		if (PointNeighbors.IsSinglePoint())
		{
			// We are on a zoom curve, or exactly on a corner. Either way, the value can be evaluated directly from the zoom curve
			for (int32 Index = 0; Index < NumParameters; ++Index)
			{
				const FRichCurve* Curve = FocusPoints[PointNeighbors.PreviousIndex].GetCurveForParameter(Index);
				if (!Curve)
				{
					return false;
				}
				
				OutBlendedParameters.Add(Curve->Eval(InZoom));
			}

			return true;
		}
		
		if (CurveNeighbors.IsSinglePoint())
		{
			// We are on one of the focus curves, so return the value evaluated on the focus curves at the specified focus
			for (int32 Index = 0; Index < NumParameters; ++Index)
			{
				const FRichCurve* Curve = FocusCurves[CurveNeighbors.PreviousIndex].GetCurveForParameter(Index);
				if (!Curve)
				{
					return false;
				}
				
				OutBlendedParameters.Add(Curve->Eval(InFocus));
			}

			return true;
		}

		const float X0 = FocusCurves[CurveNeighbors.PreviousIndex].Zoom;
		const float X1 = FocusCurves[CurveNeighbors.NextIndex].Zoom;
		const float Y0 = FocusPoints[PointNeighbors.PreviousIndex].Focus;
		const float Y1 = FocusPoints[PointNeighbors.NextIndex].Focus;
		
		for (int32 Index = 0; Index < NumParameters; ++Index)
		{
			const FRichCurve* X0Curve = FocusPoints[PointNeighbors.PreviousIndex].GetCurveForParameter(Index);
			const FRichCurve* X1Curve = FocusPoints[PointNeighbors.NextIndex].GetCurveForParameter(Index);
			const FRichCurve* Y0Curve = FocusCurves[CurveNeighbors.PreviousIndex].GetCurveForParameter(Index);
			const FRichCurve* Y1Curve = FocusCurves[CurveNeighbors.NextIndex].GetCurveForParameter(Index);

			if (!X0Curve || !X1Curve || !Y0Curve || !Y1Curve)
			{
				return false;
			}
			
			FCoonsPatch CoonsPatch(*X0Curve, *X1Curve, *Y0Curve, *Y1Curve, X0, X1, Y0, Y1);
			OutBlendedParameters.Add(CoonsPatch.Blend(FVector2D(InZoom, InFocus)));
		}
		
		return true;
	}

	/** Gets the distortion info for the specified focus point and zoom. By default, returns an unset TOptional */
	template<typename TableType>
	TOptional<FDistortionInfo> GetDistortionInfo(const typename TableType::FocusPointType& FocusPoint, float Zoom) { return TOptional<FDistortionInfo>(); }

	/** Template specialization of GetDistortionInfo for FDistortionTable */
	template<>
	TOptional<FDistortionInfo> GetDistortionInfo<FDistortionTable>(const FDistortionFocusPoint& FocusPoint, float Zoom);
		
	/** Gets the pointers to displacement maps for the specified focus point and zoom. Does nothing by default */
	template<typename TableType>
	void GetDisplacementMaps(const typename TableType::FocusPointType& FocusPoint, float Zoom, UTextureRenderTarget2D*& OutUndistortedMap, UTextureRenderTarget2D*& OutDistortedMap) { }
		
	/** Template specialization of GetDisplacementMaps for FSTMapTable */
	template<>
	void GetDisplacementMaps<FSTMapTable>(const FSTMapFocusPoint& FocusPoint, float Zoom, UTextureRenderTarget2D*& OutUndistortedMap, UTextureRenderTarget2D*& OutDistortedMap);

	/** Finds the focus points for the specified previous and next focus, with special handling for the case where there is no previous or next focus point */
	template<typename TableType>
	LensDataTableUtils::FPointNeighbors GetOnCurveFocusPoints(const TConstArrayView<typename TableType::FocusPointType>& FocusPoints, float PrevFocus, float NextFocus)
	{
		LensDataTableUtils::FPointNeighbors Result;
		
		int32 PrevIndex = FocusPoints.IndexOfByPredicate([PrevFocus](const typename TableType::FocusPointType& FocusPoint)
		{
			return FMath::IsNearlyEqual(FocusPoint.Focus, PrevFocus);
		});
		int32 NextIndex = FocusPoints.IndexOfByPredicate([NextFocus](const typename TableType::FocusPointType& FocusPoint)
		{
			return FMath::IsNearlyEqual(FocusPoint.Focus, NextFocus);
		});

		if (FocusPoints.IsValidIndex(PrevIndex) || FocusPoints.IsValidIndex(NextIndex))
		{
			// If one of the endpoints does not exist, assume the curve is flat, and set both endpoints to the same focus point
			Result.PreviousIndex = FocusPoints.IsValidIndex(PrevIndex) ? PrevIndex : NextIndex;
			Result.NextIndex = FocusPoints.IsValidIndex(NextIndex) ? NextIndex : PrevIndex;
		}

		return Result;
	}

	/** Blended results from a distortion map blend. Optionals are set when the blend input parameters indicated those blended values should be computed */
	struct FDistortionMapBlendResults
	{
		/** Indicates that a distortion map blend successfully occurred */
		bool bValid = false;

		/** The shader blending parameters, if they were computed for the blend */
		TOptional<FDisplacementMapBlendingParams> BlendingParams = TOptional<FDisplacementMapBlendingParams>();

		/** The blended distortion parameters, if they were computed for the blend */
		TOptional<FDistortionInfo> BlendedDistortionParams = TOptional<FDistortionInfo>();

		/** The undistorted maps for the blending, if they generated */
		TOptional<TArray<UTextureRenderTarget2D*>> UndistortedMaps = TOptional<TArray<UTextureRenderTarget2D*>>();

		/** The distorted maps for the blending, if they generated */
		TOptional<TArray<UTextureRenderTarget2D*>> DistortedMaps = TOptional<TArray<UTextureRenderTarget2D*>>();
		
		/** The blended overscan, if it was computed for the blend */
		TOptional<float> BlendedOverscan = TOptional<float>();
	};

	/**
	 * Parameters for a distortion map blend that configure what values are computed for the blend and how to retrieve the necessary
	 * blending values from each point being blended
	 */
	template<typename TableType>
	struct FDistortionMapBlendParams
	{
		using FocusPointType = typename TableType::FocusPointType;
		using FocusCurveType = typename TableType::FocusCurveType;
		
		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FGetInterpolatedImageCenter,
			float /* InFocus */,
			float /* InZoom */,
			FImageCenterInfo& /* OutImageCenterInfo */);

		DECLARE_DELEGATE_RetVal_ThreeParams(bool, FGetInterpolatedFocalLength,
			float /* InFocus */,
			float /* InZoom */,
			FFocalLengthInfo& /* OutFocalLengthInfo */);

		DECLARE_DELEGATE_ThreeParams(FProcessDisplacementMaps,
			const FLensDistortionState& /* InDistortionState */,
			UTextureRenderTarget2D* /* OutUndistortionMap */,
			UTextureRenderTarget2D* /* OutDistortionMap */);

		DECLARE_DELEGATE_RetVal_ThreeParams(float, FGetOverscan,
			const FocusPointType& /* InFocusPoint */,
			float /* InZoom */,
			const FLensDistortionState& /* InDistortionState */);

		/** Callback to use when the interpolated image center needs to be computed */
		FGetInterpolatedImageCenter GetInterpolatedImageCenter;
		
		/** Callback to use when the interpolated image center needs to be computed */
		FGetInterpolatedFocalLength GetInterpolatedFocalLength;
		
		FProcessDisplacementMaps ProcessDisplacementMaps;

		/** Callback to compute the overscan for a specific distortion state */
		FGetOverscan GetOverscan;
		
		/** Indicates that shader blending parameters (Results.BlendingParams) should be calculated */
		bool bGenerateBlendingParams = false;

		/** Indicates whether distortion parameters (Results.BlendedDistortionParams) should be interpolated at the focus and zoom */
		bool bGenerateBlendedDistortionParams = false;

		/** Indicates whether distortion maps (Results.UndistortedMaps and Results.DistortedMaps) should be generated */
		bool bGenerateDistortionMaps = false;

		/** Indicates whether overscan (Results.BlendedOverscan) should be generated */ 
		bool bGenerateBlendedOverscan = false;

		/** The default distortion parameters to use for cases when there is no distortion data in the table being interpolated over */
		TArray<float> DefaultDistortionParams;
		
		/** When supplied, list of render targets to write the undistorted displacement maps to */
		TArray<UTextureRenderTarget2D*> UndistortedMaps;

		/** When supplied, list of render targets to write the distorted displacement maps to */
		TArray<UTextureRenderTarget2D*> DistortedMaps;

		/** Initializes the blend results as needed based on the state of the parameters. OutResults.bValid will be false if an issue was found with the parameters */
		void InitializeResults(float InFocus, float InZoom, FDistortionMapBlendResults& OutResults) const { }
	};

	template<>
	void FDistortionMapBlendParams<FDistortionTable>::InitializeResults(float InFocus, float InZoom, FDistortionMapBlendResults& OutResults) const;

	template<>
	void FDistortionMapBlendParams<FSTMapTable>::InitializeResults(float InFocus, float InZoom, FDistortionMapBlendResults& OutResults) const;
	
	/** Performs a Coons patch blend on the distortion map parameters for the specified table at the specified focus and zoom */
	template<typename TableType>
	FDistortionMapBlendResults DistortionMapBlend(const TableType& Table, float InFocus, float InZoom, const FDistortionMapBlendParams<TableType>& InParams)
	{
		FDistortionMapBlendResults Results;
		
		using FocusPointType = typename TableType::FocusPointType;
		using FocusCurveType = typename TableType::FocusCurveType;
		
		TConstArrayView<FocusPointType> FocusPoints = Table.GetFocusPoints();
		TConstArrayView<FocusCurveType> FocusCurves = Table.GetFocusCurves();
		
		if (FocusPoints.Num() <= 0)
		{
			return Results;
		}

		InParams.InitializeResults(InFocus, InZoom, Results);
		if (!Results.bValid)
		{
			// Something is wrong with the input parameters, and they could not initialize the results, so abort
			return Results;
		}
		
		const LensDataTableUtils::FPointNeighbors PointNeighbors = LensDataTableUtils::FindFocusPoints(InFocus, FocusPoints);
		const LensDataTableUtils::FPointNeighbors CurveNeighbors = LensDataTableUtils::FindFocusCurves(InZoom, FocusCurves);
			
		const FocusPointType& PrevFocusPoint = FocusPoints[PointNeighbors.PreviousIndex];
		const FocusPointType& NextFocusPoint = FocusPoints[PointNeighbors.NextIndex];
		const FocusCurveType& PrevFocusCurve = FocusCurves[CurveNeighbors.PreviousIndex];
		const FocusCurveType& NextFocusCurve = FocusCurves[CurveNeighbors.NextIndex];

		struct FCurveEndpoints
		{
			FDistortionInfo X0;
			FDistortionInfo X1;
		};

		// Helper that can retrieve distortion info for the endpoints of curves, and correctly handles the case where
		// no data exists for one or both endpoints of the curve.
		auto CreateCurveEndpoints = [&InParams](TOptional<FDistortionInfo> InX0, TOptional<FDistortionInfo> InX1)
		{
			if (InX0.IsSet() && InX1.IsSet())
			{
				return FCurveEndpoints { InX0.GetValue(), InX1.GetValue() };
			}
			
			if (InX0.IsSet())
			{
				return FCurveEndpoints { InX0.GetValue(), InX0.GetValue() };
			}

			if (InX1.IsSet())
			{
				return FCurveEndpoints { InX1.GetValue(), InX1.GetValue() };
			}

			FCurveEndpoints DefaultEndpoints;
			
			DefaultEndpoints.X0.Parameters = InParams.DefaultDistortionParams;
			DefaultEndpoints.X1.Parameters = InParams.DefaultDistortionParams;

			return DefaultEndpoints;
		};

		if (PointNeighbors.IsSinglePoint() && CurveNeighbors.IsSinglePoint() && PrevFocusPoint.MapBlendingCurve.FindKey(PrevFocusCurve.Zoom) != FKeyHandle::Invalid())
		{
			// We are at the intersection of two curves, with a defined point. Simply return the values at that point
			
			FLensDistortionState InterpolationState;

			FDistortionInfo DefaultInfo;
			DefaultInfo.Parameters = InParams.DefaultDistortionParams;
			
			InterpolationState.DistortionInfo = GetDistortionInfo<TableType>(PrevFocusPoint, PrevFocusCurve.Zoom).Get(DefaultInfo);

			if (InParams.GetInterpolatedImageCenter.IsBound())
			{
				InParams.GetInterpolatedImageCenter.Execute(PrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationState.ImageCenter);
			}

			if (InParams.GetInterpolatedFocalLength.IsBound())
			{
				InParams.GetInterpolatedFocalLength.Execute(PrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationState.FocalLengthInfo);
			}
			
			if (Results.BlendingParams.IsSet())
			{
				Results.BlendingParams->BlendType = EDisplacementMapBlendType::OneFocusOneZoom;
				Results.BlendingParams->States[0] = InterpolationState;
			}

			if (Results.BlendedDistortionParams.IsSet())
			{
				Results.BlendedDistortionParams = InterpolationState.DistortionInfo;
			}

			if (Results.UndistortedMaps.IsSet())
			{
				GetDisplacementMaps<TableType>(PrevFocusPoint, PrevFocusCurve.Zoom, (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
				
				if (InParams.ProcessDisplacementMaps.IsBound())
				{
					InParams.ProcessDisplacementMaps.Execute(InterpolationState, (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
				}
			}

			if (Results.BlendedOverscan.IsSet())
			{
				Results.BlendedOverscan = InParams.GetOverscan.Execute(PrevFocusPoint, PrevFocusCurve.Zoom, InterpolationState);
			}
		}
		else if (PointNeighbors.IsSinglePoint())
		{
			FTangentBezierCurve BlendCurve = GetTangentCurve(PrevFocusPoint.MapBlendingCurve, InZoom, PrevFocusCurve.Zoom, NextFocusCurve.Zoom);
			FCurveEndpoints CurveEndpoints = CreateCurveEndpoints(
				GetDistortionInfo<TableType>(PrevFocusPoint, BlendCurve.X0),
				GetDistortionInfo<TableType>(PrevFocusPoint, BlendCurve.X1));
			
			FLensDistortionState InterpolationStates[2];
			InterpolationStates[0].DistortionInfo = CurveEndpoints.X0;
			InterpolationStates[1].DistortionInfo = CurveEndpoints.X1;

			if (InParams.GetInterpolatedImageCenter.IsBound())
			{
				InParams.GetInterpolatedImageCenter.Execute(PrevFocusPoint.Focus, BlendCurve.X0, InterpolationStates[0].ImageCenter);
				InParams.GetInterpolatedImageCenter.Execute(PrevFocusPoint.Focus, BlendCurve.X1, InterpolationStates[1].ImageCenter);
			}

			if (InParams.GetInterpolatedFocalLength.IsBound())
			{
				InParams.GetInterpolatedFocalLength.Execute(PrevFocusPoint.Focus, BlendCurve.X0, InterpolationStates[0].FocalLengthInfo);
				InParams.GetInterpolatedFocalLength.Execute(PrevFocusPoint.Focus, BlendCurve.X1, InterpolationStates[1].FocalLengthInfo);
			}
			
			if (Results.BlendingParams.IsSet())
			{
				Results.BlendingParams->BlendType = EDisplacementMapBlendType::OneFocusTwoZoom;

				Results.BlendingParams->PatchCorners[0] = FDisplacementMapBlendPatchCorner(BlendCurve.X0, PrevFocusPoint.Focus, BlendCurve.Tangent0, 0.0);
				Results.BlendingParams->PatchCorners[1] = FDisplacementMapBlendPatchCorner(BlendCurve.X1, PrevFocusPoint.Focus, BlendCurve.Tangent1, 0.0);

				Results.BlendingParams->States[0] = InterpolationStates[0];
				Results.BlendingParams->States[1] = InterpolationStates[1];
			}

			if (Results.BlendedDistortionParams.IsSet())
			{
				Results.BlendedDistortionParams->Parameters = BlendCurve.MultiEval(InZoom, InterpolationStates[0].DistortionInfo.Parameters, InterpolationStates[0].DistortionInfo.Parameters);
			}
			
			if (Results.UndistortedMaps.IsSet())
			{
				GetDisplacementMaps<TableType>(PrevFocusPoint, BlendCurve.X0, (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
				GetDisplacementMaps<TableType>(PrevFocusPoint, BlendCurve.X1, (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);
				
				if (InParams.ProcessDisplacementMaps.IsBound())
				{
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[0], (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[1], (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);
				}
			}

			if (Results.BlendedOverscan.IsSet())
			{
				BlendCurve.Y0 = InParams.GetOverscan.Execute(PrevFocusPoint, BlendCurve.X0, InterpolationStates[0]);
				BlendCurve.Y1 = InParams.GetOverscan.Execute(PrevFocusPoint, BlendCurve.X1, InterpolationStates[1]);
				
				Results.BlendedOverscan = BlendCurve.Eval(InZoom);
			}
		}
		else if (CurveNeighbors.IsSinglePoint())
		{
			FTangentBezierCurve BlendCurve = GetTangentCurve(PrevFocusCurve.MapBlendingCurve, InFocus, PrevFocusPoint.Focus, NextFocusPoint.Focus);

			// Since the data is stored on the focus points and not the focus curves, we must find the focus points that have the actual defined point
			// corresponding to the zoom of the focus curve we are on
			LensDataTableUtils::FPointNeighbors OnCurvePointNeighbors = GetOnCurveFocusPoints<TableType>(FocusPoints, BlendCurve.X0, BlendCurve.X1);

			const FocusPointType& OnCurvePrevFocusPoint = FocusPoints[OnCurvePointNeighbors.PreviousIndex];
			const FocusPointType& OnCurveNextFocusPoint = FocusPoints[OnCurvePointNeighbors.NextIndex];

			FCurveEndpoints CurveEndpoints = CreateCurveEndpoints(
				GetDistortionInfo<TableType>(OnCurvePrevFocusPoint, PrevFocusCurve.Zoom),
				GetDistortionInfo<TableType>(OnCurveNextFocusPoint, PrevFocusCurve.Zoom));
			
			FLensDistortionState InterpolationStates[2];
			InterpolationStates[0].DistortionInfo = CurveEndpoints.X0;
			InterpolationStates[1].DistortionInfo = CurveEndpoints.X1;

			if (InParams.GetInterpolatedImageCenter.IsBound())
			{
				InParams.GetInterpolatedImageCenter.Execute(OnCurvePrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[0].ImageCenter);
				InParams.GetInterpolatedImageCenter.Execute(OnCurveNextFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[1].ImageCenter);
			}

			if (InParams.GetInterpolatedFocalLength.IsBound())
			{
				InParams.GetInterpolatedFocalLength.Execute(OnCurvePrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[0].FocalLengthInfo);
				InParams.GetInterpolatedFocalLength.Execute(OnCurveNextFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[1].FocalLengthInfo);
			}

			if (Results.BlendingParams.IsSet())
			{
				Results.BlendingParams->BlendType = EDisplacementMapBlendType::TwoFocusOneZoom;

				Results.BlendingParams->PatchCorners[0] = FDisplacementMapBlendPatchCorner(PrevFocusCurve.Zoom, OnCurvePrevFocusPoint.Focus, 0.0f, BlendCurve.Tangent0);
				Results.BlendingParams->PatchCorners[1] = FDisplacementMapBlendPatchCorner(PrevFocusCurve.Zoom, OnCurveNextFocusPoint.Focus, 0.0f, BlendCurve.Tangent1);

				Results.BlendingParams->States[0] = InterpolationStates[0];
				Results.BlendingParams->States[1] = InterpolationStates[1];
			}
			
			if (Results.BlendedDistortionParams.IsSet())
			{
				Results.BlendedDistortionParams->Parameters = BlendCurve.MultiEval(InFocus, InterpolationStates[0].DistortionInfo.Parameters, InterpolationStates[1].DistortionInfo.Parameters);
			}
			
			if (Results.UndistortedMaps.IsSet())
			{
				GetDisplacementMaps<TableType>(OnCurvePrevFocusPoint, PrevFocusCurve.Zoom, (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
				GetDisplacementMaps<TableType>(OnCurveNextFocusPoint, PrevFocusCurve.Zoom, (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);

				if (InParams.ProcessDisplacementMaps.IsBound())
				{
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[0], (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[1], (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);
				}
			}

			if (Results.BlendedOverscan.IsSet())
			{
				BlendCurve.Y0 = InParams.GetOverscan.Execute(OnCurvePrevFocusPoint, PrevFocusCurve.Zoom, InterpolationStates[0]);
				BlendCurve.Y1 = InParams.GetOverscan.Execute(OnCurveNextFocusPoint, PrevFocusCurve.Zoom, InterpolationStates[1]);
							
				Results.BlendedOverscan = BlendCurve.Eval(InFocus);
			}
		}
		else
		{
			// Otherwise, we are somewhere in the middle of the patch, and must do a full Coons patch blend

			// We will definitely have four curves that intersect in a quad patch within which a Coons patch interpolation can be performed.
			// However, the four corners of the patch may not all be defined and thus won't have tangent or distortion parameter data to interpolate
			// from. These 'virtual' points must be approximated before the final Coons patch interpolation can be done. The curves can be used to
			// interpolate the 'virtual' points from the nearest defined points, and from there, an approximate Coons patch can be assembled from which
			// the final interpolation can be performed.
			
			FTangentBezierCoonsPatch CoonsPatch
			(
				FTangentBezierCoonsPatch::FPatchCurve(GetTangentCurve(PrevFocusPoint.MapBlendingCurve, InZoom, PrevFocusCurve.Zoom, NextFocusCurve.Zoom), PrevFocusPoint.Focus),
				FTangentBezierCoonsPatch::FPatchCurve(GetTangentCurve(NextFocusPoint.MapBlendingCurve, InZoom, PrevFocusCurve.Zoom, NextFocusCurve.Zoom), NextFocusPoint.Focus),
				FTangentBezierCoonsPatch::FPatchCurve(GetTangentCurve(PrevFocusCurve.MapBlendingCurve, InFocus, PrevFocusPoint.Focus, NextFocusPoint.Focus), PrevFocusCurve.Zoom),
				FTangentBezierCoonsPatch::FPatchCurve(GetTangentCurve(NextFocusCurve.MapBlendingCurve, InFocus, PrevFocusPoint.Focus, NextFocusPoint.Focus), NextFocusCurve.Zoom)
			);

			// Interpolate the tangents needed at the potentially virtual patch corners from the curves from definite points
			// Distortion parameter curves use the focus/zoom values on the curves as the curves' y values, so we must
			// mirror this in order to properly interpolate the tangents at the virtual points
			CoonsPatch.XCurves[0].TangentCurve.Y0 = CoonsPatch.XCurves[0].TangentCurve.X0;
			CoonsPatch.XCurves[0].TangentCurve.Y1 = CoonsPatch.XCurves[0].TangentCurve.X1;
			CoonsPatch.XCurves[1].TangentCurve.Y0 = CoonsPatch.XCurves[1].TangentCurve.X0;
			CoonsPatch.XCurves[1].TangentCurve.Y1 = CoonsPatch.XCurves[1].TangentCurve.X1;
			CoonsPatch.YCurves[0].TangentCurve.Y0 = CoonsPatch.YCurves[0].TangentCurve.X0;
			CoonsPatch.YCurves[0].TangentCurve.Y1 = CoonsPatch.YCurves[0].TangentCurve.X1;
			CoonsPatch.YCurves[1].TangentCurve.Y0 = CoonsPatch.YCurves[1].TangentCurve.X0;
			CoonsPatch.YCurves[1].TangentCurve.Y1 = CoonsPatch.YCurves[1].TangentCurve.X1;
			
			float XTangents[4] =
			{
				CoonsPatch.XCurves[0].TangentCurve.EvalTangent(PrevFocusCurve.Zoom),
				CoonsPatch.XCurves[0].TangentCurve.EvalTangent(NextFocusCurve.Zoom),
				CoonsPatch.XCurves[1].TangentCurve.EvalTangent(NextFocusCurve.Zoom),
				CoonsPatch.XCurves[1].TangentCurve.EvalTangent(PrevFocusCurve.Zoom)
			};
		
			float YTangents[4] =
			{
				CoonsPatch.YCurves[0].TangentCurve.EvalTangent(PrevFocusPoint.Focus),
				CoonsPatch.YCurves[1].TangentCurve.EvalTangent(PrevFocusPoint.Focus),
				CoonsPatch.YCurves[1].TangentCurve.EvalTangent(NextFocusPoint.Focus),
				CoonsPatch.YCurves[0].TangentCurve.EvalTangent(NextFocusPoint.Focus)
			};

			LensDataTableUtils::FPointNeighbors PrevCurvePointNeighbors = GetOnCurveFocusPoints<TableType>(FocusPoints, CoonsPatch.YCurves[0].TangentCurve.X0, CoonsPatch.YCurves[0].TangentCurve.X1);
			LensDataTableUtils::FPointNeighbors NextCurvePointNeighbors = GetOnCurveFocusPoints<TableType>(FocusPoints, CoonsPatch.YCurves[1].TangentCurve.X0, CoonsPatch.YCurves[1].TangentCurve.X1);
			
			const FocusPointType& PrevCurvePrevFocusPoint = FocusPoints[PrevCurvePointNeighbors.PreviousIndex];
			const FocusPointType& PrevCurveNextFocusPoint = FocusPoints[PrevCurvePointNeighbors.NextIndex];
			const FocusPointType& NextCurvePrevFocusPoint = FocusPoints[NextCurvePointNeighbors.PreviousIndex];
			const FocusPointType& NextCurveNextFocusPoint = FocusPoints[NextCurvePointNeighbors.NextIndex];

			FLensDistortionState InterpolationStates[4];
			FDistortionInfo InterpolatedDistortionInfo;

			// Compute the virtual corners of the patch via interpolation from the nearest defined points on each edge curve
			FCurveEndpoints XCurveEndpoints[2] =
            {
                CreateCurveEndpoints(
                	GetDistortionInfo<TableType>(PrevFocusPoint, CoonsPatch.XCurves[0].TangentCurve.X0),
                	GetDistortionInfo<TableType>(PrevFocusPoint, CoonsPatch.XCurves[0].TangentCurve.X1)
                ),
                CreateCurveEndpoints(
                	GetDistortionInfo<TableType>(NextFocusPoint, CoonsPatch.XCurves[1].TangentCurve.X0),
					GetDistortionInfo<TableType>(NextFocusPoint, CoonsPatch.XCurves[1].TangentCurve.X1)
                )
            };

			FCurveEndpoints YCurveEndpoints[2] =
			{
				CreateCurveEndpoints(
					GetDistortionInfo<TableType>(PrevCurvePrevFocusPoint, PrevFocusCurve.Zoom),
					GetDistortionInfo<TableType>(PrevCurveNextFocusPoint, PrevFocusCurve.Zoom)
				),
				CreateCurveEndpoints(
					GetDistortionInfo<TableType>(NextCurvePrevFocusPoint, NextFocusCurve.Zoom),
					GetDistortionInfo<TableType>(NextCurveNextFocusPoint, NextFocusCurve.Zoom)
				)
			};

			const int32 NumParams = InParams.DefaultDistortionParams.Num();
			InterpolationStates[0].DistortionInfo.Parameters.AddZeroed(NumParams);
			InterpolationStates[1].DistortionInfo.Parameters.AddZeroed(NumParams);
			InterpolationStates[2].DistortionInfo.Parameters.AddZeroed(NumParams);
			InterpolationStates[3].DistortionInfo.Parameters.AddZeroed(NumParams);
			InterpolatedDistortionInfo.Parameters.AddZeroed(NumParams);
			
			for (int Index = 0; Index < NumParams; ++Index)
			{
				CoonsPatch.XCurves[0].TangentCurve.Y0 = XCurveEndpoints[0].X0.Parameters[Index];
				CoonsPatch.XCurves[0].TangentCurve.Y1 = XCurveEndpoints[0].X1.Parameters[Index];
				CoonsPatch.XCurves[1].TangentCurve.Y0 = XCurveEndpoints[1].X0.Parameters[Index];
				CoonsPatch.XCurves[1].TangentCurve.Y1 = XCurveEndpoints[1].X1.Parameters[Index];

				CoonsPatch.YCurves[0].TangentCurve.Y0 = YCurveEndpoints[0].X0.Parameters[Index];
				CoonsPatch.YCurves[0].TangentCurve.Y1 = YCurveEndpoints[0].X1.Parameters[Index];
				CoonsPatch.YCurves[1].TangentCurve.Y0 = YCurveEndpoints[1].X0.Parameters[Index];
				CoonsPatch.YCurves[1].TangentCurve.Y1 = YCurveEndpoints[1].X1.Parameters[Index];

				InterpolationStates[0].DistortionInfo.Parameters[Index] = CoonsPatch.Blend(FVector2D(PrevFocusCurve.Zoom, PrevFocusPoint.Focus));
				InterpolationStates[1].DistortionInfo.Parameters[Index] = CoonsPatch.Blend(FVector2D(NextFocusCurve.Zoom, PrevFocusPoint.Focus));
				InterpolationStates[2].DistortionInfo.Parameters[Index] = CoonsPatch.Blend(FVector2D(NextFocusCurve.Zoom, NextFocusPoint.Focus));
				InterpolationStates[3].DistortionInfo.Parameters[Index] = CoonsPatch.Blend(FVector2D(PrevFocusCurve.Zoom, NextFocusPoint.Focus));
				
				InterpolatedDistortionInfo.Parameters[Index] = CoonsPatch.Blend(FVector2D(InZoom, InFocus));
			}
			
			if (InParams.GetInterpolatedImageCenter.IsBound())
			{
				InParams.GetInterpolatedImageCenter.Execute(PrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[0].ImageCenter);
				InParams.GetInterpolatedImageCenter.Execute(PrevFocusPoint.Focus, NextFocusCurve.Zoom, InterpolationStates[1].ImageCenter);
				InParams.GetInterpolatedImageCenter.Execute(NextFocusPoint.Focus, NextFocusCurve.Zoom, InterpolationStates[2].ImageCenter);
				InParams.GetInterpolatedImageCenter.Execute(NextFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[3].ImageCenter);
			}
			
			if (InParams.GetInterpolatedFocalLength.IsBound())
			{
				InParams.GetInterpolatedFocalLength.Execute(PrevFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[0].FocalLengthInfo);
				InParams.GetInterpolatedFocalLength.Execute(PrevFocusPoint.Focus, NextFocusCurve.Zoom, InterpolationStates[1].FocalLengthInfo);
				InParams.GetInterpolatedFocalLength.Execute(NextFocusPoint.Focus, NextFocusCurve.Zoom, InterpolationStates[2].FocalLengthInfo);
				InParams.GetInterpolatedFocalLength.Execute(NextFocusPoint.Focus, PrevFocusCurve.Zoom, InterpolationStates[3].FocalLengthInfo);
			}
			
			if (Results.BlendingParams.IsSet())
			{
				Results.BlendingParams->BlendType = EDisplacementMapBlendType::TwoFocusTwoZoom;

				Results.BlendingParams->PatchCorners[0] = FDisplacementMapBlendPatchCorner(PrevFocusCurve.Zoom, PrevFocusPoint.Focus, XTangents[0], YTangents[0]);
				Results.BlendingParams->PatchCorners[1] = FDisplacementMapBlendPatchCorner(NextFocusCurve.Zoom, PrevFocusPoint.Focus, XTangents[1], YTangents[1]);
				Results.BlendingParams->PatchCorners[2] = FDisplacementMapBlendPatchCorner(NextFocusCurve.Zoom, NextFocusPoint.Focus, XTangents[2], YTangents[2]);
				Results.BlendingParams->PatchCorners[3] = FDisplacementMapBlendPatchCorner(PrevFocusCurve.Zoom, NextFocusPoint.Focus, XTangents[3], YTangents[3]);

				Results.BlendingParams->States[0] = InterpolationStates[0];
				Results.BlendingParams->States[1] = InterpolationStates[1];
				Results.BlendingParams->States[2] = InterpolationStates[2];
				Results.BlendingParams->States[3] = InterpolationStates[3];
			}
			
			if (Results.BlendedDistortionParams.IsSet())
			{
				Results.BlendedDistortionParams = InterpolatedDistortionInfo;
			}
			
			if (Results.UndistortedMaps.IsSet())
			{
				GetDisplacementMaps<TableType>(PrevFocusPoint, PrevFocusCurve.Zoom, (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
				GetDisplacementMaps<TableType>(PrevFocusPoint, NextFocusCurve.Zoom, (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);
				GetDisplacementMaps<TableType>(NextFocusPoint, NextFocusCurve.Zoom, (*Results.UndistortedMaps)[2], (*Results.DistortedMaps)[2]);
				GetDisplacementMaps<TableType>(NextFocusPoint, PrevFocusCurve.Zoom, (*Results.UndistortedMaps)[3], (*Results.DistortedMaps)[3]);

				if (InParams.ProcessDisplacementMaps.IsBound())
				{
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[0], (*Results.UndistortedMaps)[0], (*Results.DistortedMaps)[0]);
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[1], (*Results.UndistortedMaps)[1], (*Results.DistortedMaps)[1]);
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[2], (*Results.UndistortedMaps)[2], (*Results.DistortedMaps)[2]);
					InParams.ProcessDisplacementMaps.Execute(InterpolationStates[3], (*Results.UndistortedMaps)[3], (*Results.DistortedMaps)[3]);
				}
			}

			if (Results.BlendedOverscan.IsSet())
			{
				// Since overscan is computed from distortion parameters, we only want to compute it for the distortion states for each of the potentially virtual patch corners.
				// As such, it needs its own Coons patch where each curve starts and ends at the correct corners. The interpolated tangents computed earlier can be used here
				// for each of the corners.
				
				float Overscans[4] =
				{
					InParams.GetOverscan.Execute(PrevFocusPoint, PrevFocusCurve.Zoom, InterpolationStates[0]),
					InParams.GetOverscan.Execute(PrevFocusPoint, NextFocusCurve.Zoom, InterpolationStates[1]),
					InParams.GetOverscan.Execute(NextFocusPoint, NextFocusCurve.Zoom, InterpolationStates[2]),
					InParams.GetOverscan.Execute(NextFocusPoint, PrevFocusCurve.Zoom, InterpolationStates[3]),
				};
				
				FTangentBezierCoonsPatch OverscanCoonsPatch
				(
					FTangentBezierCoonsPatch::FPatchCurve(FTangentBezierCurve(PrevFocusCurve.Zoom, NextFocusCurve.Zoom, Overscans[0], Overscans[1], XTangents[0], XTangents[1]), PrevFocusPoint.Focus),
					FTangentBezierCoonsPatch::FPatchCurve(FTangentBezierCurve(PrevFocusCurve.Zoom, NextFocusCurve.Zoom, Overscans[3], Overscans[2], XTangents[3], XTangents[2]), NextFocusPoint.Focus),
					FTangentBezierCoonsPatch::FPatchCurve(FTangentBezierCurve(PrevFocusPoint.Focus, NextFocusPoint.Focus, Overscans[0], Overscans[3], YTangents[0], YTangents[3]), PrevFocusCurve.Zoom),
					FTangentBezierCoonsPatch::FPatchCurve(FTangentBezierCurve(PrevFocusPoint.Focus, NextFocusPoint.Focus, Overscans[1], Overscans[2], YTangents[1], YTangents[2]), NextFocusCurve.Zoom)
				);
				
				Results.BlendedOverscan = OverscanCoonsPatch.Blend(FVector2D(InZoom, InFocus));
			}
		}

		return Results;
	}
};
