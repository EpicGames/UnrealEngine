// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/AnamorphicLensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnamorphicLensModel)

UScriptStruct* UAnamorphicLensModel::GetParameterStruct() const
{
	return FAnamorphicDistortionParameters::StaticStruct();
}

FName UAnamorphicLensModel::GetModelName() const
{
	return FName("3DE4 Anamorphic Standard Degree 4");
}

FName UAnamorphicLensModel::GetShortModelName() const
{
	return TEXT("3DE4 Anamorphic Standard Degree 4");
}
