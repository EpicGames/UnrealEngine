// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/BrownConradyDULensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrownConradyDULensModel)

UScriptStruct* UBrownConradyDULensModel::GetParameterStruct() const
{
	// Use the same parameter struct as UD model
	return FBrownConradyUDDistortionParameters::StaticStruct();
}

FName UBrownConradyDULensModel::GetModelName() const
{
	return FName("Brown-Conrady D-U");
}

FName UBrownConradyDULensModel::GetShortModelName() const
{
	return FName("Brown-Conrady D-U");
}