// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/BrownConradyUDLensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrownConradyUDLensModel)

UScriptStruct* UBrownConradyUDLensModel::GetParameterStruct() const
{
	return FBrownConradyUDDistortionParameters::StaticStruct();
}

FName UBrownConradyUDLensModel::GetModelName() const 
{ 
	return FName("Brown-Conrady U-D"); 
}

FName UBrownConradyUDLensModel::GetShortModelName() const
{
	return TEXT("Brown-Conrady U-D");
}