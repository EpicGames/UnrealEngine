// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/LensModel.h"


#include "LensDistortionModelHandlerBase.h"
#include "Logging/LogMacros.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensModel)


DEFINE_LOG_CATEGORY_STATIC(LogLensModel, Log, All);

#if WITH_EDITOR

TArray<FText> ULensModel::GetParameterDisplayNames() const
{
	TArray<FText> ParameterNames;

	UScriptStruct* TypeStruct = GetParameterStruct();

	for (TFieldIterator<FProperty> It(TypeStruct); It; ++It)
	{
		if (FFloatProperty* Property = CastField<FFloatProperty>(*It))
		{
			ParameterNames.Add(Property->GetDisplayNameText());
		}
		else
		{
			UE_LOGF(LogLensModel, Warning, "Property '%ls' was skipped because its type was not float", *(It->GetNameCPP()));
		}
	}

	return ParameterNames;
}

#endif //WITH_EDITOR

void ULensModel::GetDefaultParameterArray(TArray<float>& OutParameterValues) const
{
	FStructOnScope StructOnScope{ GetParameterStruct() };
	ToArray<FStructOnScope>(StructOnScope, OutParameterValues);
}

uint32 ULensModel::GetNumParameters() const
{
	UScriptStruct* TypeStruct = GetParameterStruct();

	uint32 NumParameters = 0;
	for (TFieldIterator<FProperty> It(TypeStruct); It; ++It)
	{
		if (FFloatProperty* Property = CastField<FFloatProperty>(*It))
		{
			++NumParameters;
		}
		else
		{
			UE_LOGF(LogLensModel, Warning, "Property '%ls' was skipped because its type was not float", *(It->GetNameCPP()));
		}
	}

	return NumParameters;
}

TSubclassOf<ULensDistortionModelHandlerBase> ULensModel::GetHandlerClass(TSubclassOf<ULensModel> LensModel)
{
	if (LensModel)
	{
		// Find all UClasses that derive from ULensDistortionModelHandlerBase
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (It->IsChildOf(ULensDistortionModelHandlerBase::StaticClass()) && !It->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
			{
				if (It->GetDefaultObject<ULensDistortionModelHandlerBase>()->IsModelSupported(LensModel))
				{
					return *It;
				}
			}
		}
	}
	return nullptr;
}

void ULensModel::ToArray_Internal(const UScriptStruct* TypeStruct, const void* SrcData, TArray<float>& DstArray) const
{
	if (TypeStruct != GetParameterStruct())
	{
		UE_LOGF(LogLensModel, Error, "TypeStruct does not match the distortion parameter struct supported by this model");
		return;
	}

	DstArray.Reset(GetNumParameters());
	for (TFieldIterator<FProperty> It(TypeStruct); It; ++It)
	{
		if (FFloatProperty* Prop = CastField<FFloatProperty>(*It))
		{
			const float* Tmp = Prop->ContainerPtrToValuePtr<float>(SrcData);
			DstArray.Add(*Tmp);
		}
		else
		{
			UE_LOGF(LogLensModel, Warning, "Property '%ls' was skipped because its type was not float", *(It->GetNameCPP()));
		}
	}
}

void ULensModel::FromArray_Internal(UScriptStruct* TypeStruct, const TArray<float>& SrcArray, void* DstData)
{
	if (TypeStruct != GetParameterStruct())
	{
		UE_LOGF(LogLensModel, Error, "TypeStruct does not match the distortion parameter struct supported by this model");
		return;
	}

	if (SrcArray.Num() != GetNumParameters())
	{
		return;
	}

	uint32 Index = 0;
	for (TFieldIterator<FProperty> It(TypeStruct); It; ++It)
	{
		if (FFloatProperty* Prop = CastField<FFloatProperty>(*It))
		{
			Prop->SetPropertyValue_InContainer(DstData, SrcArray[Index]);
			++Index;
		}
		else
		{
			UE_LOGF(LogLensModel, Warning, "Property '%ls' was skipped because its type was not float", *(It->GetNameCPP()));
		}
	}
}
