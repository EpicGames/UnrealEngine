// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/SphericalLensModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SphericalLensModel)

UScriptStruct* USphericalLensModel::GetParameterStruct() const
{
	return FSphericalDistortionParameters::StaticStruct();
}

FName USphericalLensModel::GetModelName() const 
{ 
	return FName("Spherical Lens Model"); 
}

FName USphericalLensModel::GetShortModelName() const
{
	return TEXT("Spherical");
}
