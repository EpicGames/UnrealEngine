// Copyright Epic Games, Inc. All Rights Reserved.

#include "OverlayRendering.h"

#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Misc/MemStack.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"


/** A pixel shader that draws a crosshair centered at a given principal point */
class FCrosshairOverlayPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCrosshairOverlayPS);
	SHADER_USE_PARAMETER_STRUCT(FCrosshairOverlayPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )

		SHADER_PARAMETER(FVector2f, PrincipalPoint)
		SHADER_PARAMETER(FIntPoint, OutputTextureExtent)
		SHADER_PARAMETER(float, LengthInPixels)
		SHADER_PARAMETER(float, GapSizeInPixels)
		SHADER_PARAMETER(FVector4f, OutputColor)

		RENDER_TARGET_BINDING_SLOTS()

	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCrosshairOverlayPS, "/Plugin/CameraCalibrationCore/Private/CrosshairOverlay.usf", "MainPS", SF_Pixel);


namespace OverlayRendering
{
	bool DrawCrosshairOverlay(UTextureRenderTarget2D* OutRenderTarget, const FCrosshairOverlayParams& CrosshairParams)
	{
		if (OutRenderTarget == nullptr)
		{
			return false;
		}

		const FTextureRenderTargetResource* const DestinationTextureResource = OutRenderTarget->GameThread_GetRenderTargetResource();

		if (!DestinationTextureResource)
		{
			return false;
		}

		ENQUEUE_RENDER_COMMAND(OverlayRendering_CrosshairOverlay)(
			[DestinationTextureResource, CrosshairParams](FRHICommandListImmediate& RHICmdList)
			{
				FRDGBuilder GraphBuilder(RHICmdList);

				FCrosshairOverlayPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCrosshairOverlayPS::FParameters>();

				FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTextureResource->TextureRHI, TEXT("OutputOverlay")));
				PassParameters->OutputTextureExtent = OutputTexture->Desc.Extent;
				PassParameters->PrincipalPoint = CrosshairParams.PrincipalPoint;
				PassParameters->LengthInPixels = CrosshairParams.LengthInPixels;
				PassParameters->GapSizeInPixels = CrosshairParams.GapSizeInPixels;
				PassParameters->OutputColor = CrosshairParams.CrosshairColor;

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				const TShaderMapRef<FCrosshairOverlayPS> PixelShader(GlobalShaderMap);

				FScreenPassRenderTarget SceneColorRenderTarget(OutputTexture, ERenderTargetLoadAction::ELoad);
				PassParameters->RenderTargets[0] = SceneColorRenderTarget.GetRenderTargetBinding();

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder
					, GlobalShaderMap
					, RDG_EVENT_NAME("CrosshairOverlay")
					, PixelShader
					, PassParameters
					, FIntRect(0, 0, OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y));

				GraphBuilder.Execute();
			});

		return true;
	}
}

