// Copyright Epic Games, Inc. All Rights Reserved.

#include "SphericalLensDistortionModelHandler.h"

#include "CameraCalibrationCoreLog.h"
#include "CameraCalibrationSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Math/NumericLimits.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SphericalLensDistortionModelHandler)

// Spherical distortion shader parameter structure
BEGIN_SHADER_PARAMETER_STRUCT(FSphericalDistortionParams, )
	SHADER_PARAMETER(FVector2f, FocalLength)
	SHADER_PARAMETER(FVector2f, ImageCenter)
	SHADER_PARAMETER(float, K1)
	SHADER_PARAMETER(float, K2)
	SHADER_PARAMETER(float, K3)
	SHADER_PARAMETER(float, P1)
	SHADER_PARAMETER(float, P2)
END_SHADER_PARAMETER_STRUCT()

// Dedicated spherical distortion compute shader
class FSphericalDistortionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSphericalDistortionCS);
	SHADER_USE_PARAMETER_STRUCT(FSphericalDistortionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, ThreadIdToUV)
		SHADER_PARAMETER(float, InverseOverscan)
		SHADER_PARAMETER(float, CameraOverscan)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSphericalDistortionParams, SphericalDistortionParams)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDistortionMap)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}
};

IMPLEMENT_GLOBAL_SHADER(FSphericalDistortionCS, "/Plugin/CameraCalibrationCore/Private/SphericalDistortion.usf", "MainCS", SF_Compute);

void USphericalLensDistortionModelHandler::InitializeHandler()
{
	LensModelClass = USphericalLensModel::StaticClass();
}

FVector2D USphericalLensDistortionModelHandler::ComputeDistortedUV(const FVector2D& InScreenUV) const
{
	const FVector2D FxFy = CurrentState.FocalLengthInfo.FxFy;
	const FVector2D CxCy = CurrentState.ImageCenter.PrincipalPoint;

	if (FMath::IsNearlyZero(FxFy.X) || FMath::IsNearlyZero(FxFy.Y))
	{
		return InScreenUV;
	}

	// @todo distortion offset to replace the 0.5 below
	FVector2D X = (InScreenUV - FVector2D(0.5f, 0.5f)) / FxFy; // normalized coords (undistorted)

	const float R2 = X.X * X.X + X.Y * X.Y;
	const float R4 = R2 * R2;
	const float R6 = R4 * R2;

	const float Radial = 1.0f
		+ SphericalParameters.K1 * R2
		+ SphericalParameters.K2 * R4
		+ SphericalParameters.K3 * R6;

	const FVector2D Tangential(
		SphericalParameters.P2 * (R2 + 2.0f * X.X * X.X) + 2.0f * SphericalParameters.P1 * X.X * X.Y,
		SphericalParameters.P1 * (R2 + 2.0f * X.Y * X.Y) + 2.0f * SphericalParameters.P2 * X.X * X.Y
	);

	X = X * Radial + Tangential;

	return X * FxFy + CxCy; // distorted UV
}

FVector2D USphericalLensDistortionModelHandler::ComputeUndistortedUV(const FVector2D& DistortedUV) const
{
	const FVector2D FxFy = CurrentState.FocalLengthInfo.FxFy;
	const FVector2D CxCy = CurrentState.ImageCenter.PrincipalPoint;

	if (FMath::IsNearlyZero(FxFy.X) || FMath::IsNearlyZero(FxFy.Y))
	{
		return DistortedUV;
	}

	const FVector2D Pd = (DistortedUV - CxCy) / FxFy;

	FVector2D P = Pd; // initial guess

	constexpr int32 MaxIters = 5;

	for (int32 It = 0; It < MaxIters; ++It)
	{
		const float R2 = P.X * P.X + P.Y * P.Y;
		const float R4 = R2 * R2;
		const float R6 = R4 * R2;

		const float Radial = 1.0f
			+ SphericalParameters.K1 * R2
			+ SphericalParameters.K2 * R4
			+ SphericalParameters.K3 * R6;

		const FVector2D Tangential(
			SphericalParameters.P2 * (R2 + 2.0f * P.X * P.X) + 2.0f * SphericalParameters.P1 * P.X * P.Y,
			SphericalParameters.P1 * (R2 + 2.0f * P.Y * P.Y) + 2.0f * SphericalParameters.P2 * P.X * P.Y
		);

		const float InvRadial = FMath::IsNearlyZero(Radial, KINDA_SMALL_NUMBER) ? 1.0f : (1.0f / Radial);

		P = (Pd - Tangential) * InvRadial;
	}

	// @todo distortion offset to replace the 0.5 below
	return P * FxFy + FVector2D(0.5f, 0.5f); // undistorted UV
}

void USphericalLensDistortionModelHandler::InitDistortionMaterials()
{
	if (DistortionPostProcessMID == nullptr)
	{
		UMaterialInterface* DistortionMaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionMaterial(this->StaticClass());
		DistortionPostProcessMID = UMaterialInstanceDynamic::Create(DistortionMaterialParent, this);
	}

	if (UndistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultUndistortionDisplacementMaterial(this->StaticClass());
		UndistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	if (DistortionDisplacementMapMID == nullptr)
	{
		UMaterialInterface* MaterialParent = GetDefault<UCameraCalibrationSettings>()->GetDefaultDistortionDisplacementMaterial(this->StaticClass());
		DistortionDisplacementMapMID = UMaterialInstanceDynamic::Create(MaterialParent, this);
	}

	DistortionPostProcessMID->SetTextureParameterValue("UndistortionDisplacementMap", UndistortionDisplacementMapRT);
	DistortionPostProcessMID->SetTextureParameterValue("DistortionDisplacementMap", DistortionDisplacementMapRT);

	SetDistortionState(CurrentState);
}

void USphericalLensDistortionModelHandler::UpdateMaterialParameters()
{
	//Helper function to set material parameters of an MID
	const auto SetDistortionMaterialParameters = [this](UMaterialInstanceDynamic* const MID)
	{
		MID->SetScalarParameterValue("k1", SphericalParameters.K1);
		MID->SetScalarParameterValue("k2", SphericalParameters.K2);
		MID->SetScalarParameterValue("k3", SphericalParameters.K3);
		MID->SetScalarParameterValue("p1", SphericalParameters.P1);
		MID->SetScalarParameterValue("p2", SphericalParameters.P2);

		MID->SetScalarParameterValue("cx", CurrentState.ImageCenter.PrincipalPoint.X);
		MID->SetScalarParameterValue("cy", CurrentState.ImageCenter.PrincipalPoint.Y);

		MID->SetScalarParameterValue("fx", CurrentState.FocalLengthInfo.FxFy.X);
		MID->SetScalarParameterValue("fy", CurrentState.FocalLengthInfo.FxFy.Y);
	};

	SetDistortionMaterialParameters(UndistortionDisplacementMapMID);
	SetDistortionMaterialParameters(DistortionDisplacementMapMID);
}

void USphericalLensDistortionModelHandler::InterpretDistortionParameters()
{
	LensModelClass->GetDefaultObject<ULensModel>()->FromArray<FSphericalDistortionParameters>(CurrentState.DistortionInfo.Parameters, SphericalParameters);
}

FString USphericalLensDistortionModelHandler::GetDistortionShaderPath() const
{
	return TEXT("/Plugin/CameraCalibrationCore/Private/SphericalDistortion.usf");
}


void USphericalLensDistortionModelHandler::ExecuteDistortionShader(FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const
{
	FIntPoint DistortionMapResolution = OutDistortionMap->Desc.Extent;

	// Early out if the distortion map has invalid dimensions.
	if (DistortionMapResolution.X <= 0 || DistortionMapResolution.Y <= 0)
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"SphericalLensDistortionModelHandler: Skipping distortion shader execution due to invalid distortion map dimensions (%dx%d)", 
			DistortionMapResolution.X, DistortionMapResolution.Y);
		return;
	}

	FSphericalDistortionCS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FSphericalDistortionCS::FParameters>();
	
	ShaderParams->OutDistortionMap = GraphBuilder.CreateUAV(OutDistortionMap);
	ShaderParams->ThreadIdToUV = FVector2f(1.0f) / FVector2f(DistortionMapResolution);
	ShaderParams->InverseOverscan = InverseOverscan;
	ShaderParams->CameraOverscan = CameraOverscan;

	// Setup spherical distortion parameters

	ShaderParams->SphericalDistortionParams.ImageCenter = FVector2f(InCurrentState.ImageCenter.PrincipalPoint);
	ShaderParams->SphericalDistortionParams.FocalLength = FVector2f(InCurrentState.FocalLengthInfo.FxFy);
	
	// Spherical model requires exactly 5 parameters (K1, K2, K3, P1, P2)
	if (InCurrentState.DistortionInfo.Parameters.Num() == 5)
	{
		ShaderParams->SphericalDistortionParams.K1 = InCurrentState.DistortionInfo.Parameters[0];
		ShaderParams->SphericalDistortionParams.K2 = InCurrentState.DistortionInfo.Parameters[1];
		ShaderParams->SphericalDistortionParams.K3 = InCurrentState.DistortionInfo.Parameters[2];
		ShaderParams->SphericalDistortionParams.P1 = InCurrentState.DistortionInfo.Parameters[3];
		ShaderParams->SphericalDistortionParams.P2 = InCurrentState.DistortionInfo.Parameters[4];
	}
	else
	{
		UE_LOGF(LogCameraCalibrationCore, Warning, 
			"SphericalLensDistortionModelHandler: Invalid parameter count (%d), expected exactly 5. Using default values.", 
			InCurrentState.DistortionInfo.Parameters.Num()
		);

		ShaderParams->SphericalDistortionParams.K1 = 0.0f;
		ShaderParams->SphericalDistortionParams.K2 = 0.0f;
		ShaderParams->SphericalDistortionParams.K3 = 0.0f;
		ShaderParams->SphericalDistortionParams.P1 = 0.0f;
		ShaderParams->SphericalDistortionParams.P2 = 0.0f;
	}

	TShaderMapRef<FSphericalDistortionCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("SphericalDistortionDisplacementMap"),
		ComputeShader,
		ShaderParams,
		FIntVector(FMath::DivideAndRoundUp(DistortionMapResolution.X, 8), FMath::DivideAndRoundUp(DistortionMapResolution.Y, 8), 1));
}

