// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tables/BaseLensTable.h"
#include "LensFile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseLensTable)

FKeyHandle FBaseFocusCurve::AddPointToCurve(FRichCurve& InCurve, float InFocus, float InValue, float InputTolerance, FKeyHandle InOptionalKeyHandle)
{
	const FKeyHandle ExistingKeyHandle = SetPointInCurve(InCurve, InFocus, InValue, InputTolerance);
	if (ExistingKeyHandle != FKeyHandle::Invalid())
	{
		return ExistingKeyHandle;
	}
	
	const FKeyHandle NewKeyHandle = InCurve.AddKey(InFocus, InValue, false, InOptionalKeyHandle);
	InCurve.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_None);
	InCurve.SetKeyInterpMode(NewKeyHandle, RCIM_Linear);

	return NewKeyHandle;
}

FKeyHandle FBaseFocusCurve::SetPointInCurve(FRichCurve& InCurve, float InFocus, float InValue, float InputTolerance)
{
	const FKeyHandle KeyHandle = InCurve.FindKey(InFocus, InputTolerance);
	if (KeyHandle != FKeyHandle::Invalid())
	{
		InCurve.SetKeyValue(KeyHandle, InValue);
	}

	return KeyHandle;
}

void FBaseFocusCurve::DeletePointFromCurve(FRichCurve& InCurve, float InFocus, float InputTolerance)
{
	const FKeyHandle KeyHandle = InCurve.FindKey(InFocus, InputTolerance);
	if (KeyHandle != FKeyHandle::Invalid())
	{
		InCurve.DeleteKey(KeyHandle);
	}
}

void FBaseFocusCurve::ChangeFocusInCurve(FRichCurve& InCurve, float InExistingFocus, float InNewFocus, float InputTolerance)
{
	const FKeyHandle KeyHandle = InCurve.FindKey(InExistingFocus, InputTolerance);
	if (KeyHandle != FKeyHandle::Invalid())
	{
		InCurve.SetKeyTime(KeyHandle, InNewFocus);
	}
}

void FBaseFocusCurve::MergeFocusInCurve(FRichCurve& InCurve, float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	const FKeyHandle KeyHandle = InCurve.FindKey(InExistingFocus, InputTolerance);
	if (KeyHandle != FKeyHandle::Invalid())
	{
		const FKeyHandle CxNewHandle = InCurve.FindKey(InNewFocus, InputTolerance);
		if (CxNewHandle != FKeyHandle::Invalid() && bReplaceExisting)
		{
			InCurve.Keys[InCurve.GetIndexSafe(CxNewHandle)] = InCurve.GetKey(KeyHandle);
			InCurve.DeleteKey(KeyHandle);
		}
		else if (CxNewHandle == FKeyHandle::Invalid())
		{
			InCurve.SetKeyTime(KeyHandle, InNewFocus);	
		}
	}
}

void FBaseLensTable::CopyCurveKeys(const FRichCurve& InSourceCurve, FRichCurve& InDestCurve, TArrayView<const FKeyHandle> InKeys)
{
	for (int32 Index = 0; Index < InKeys.Num(); ++Index)
	{
		const FKeyHandle Handle = InKeys[Index];
		const int32 KeyIndex = InSourceCurve.GetIndexSafe(Handle);
		if (KeyIndex != INDEX_NONE)
		{
			InDestCurve.Keys[KeyIndex] = InSourceCurve.GetKey(Handle);
		}
	}

	InDestCurve.AutoSetTangents();
}

void FBaseLensTable::PropagateCurveValuesToCrossCurves(const FRichCurve& InCurve, float InCrossCurveTime, TFunctionRef<FRichCurve*(float)> GetCurveFn)
{
	for (auto Iter(InCurve.GetKeyIterator()); Iter; ++Iter)
	{
		if (FRichCurve* CrossCurve = GetCurveFn(Iter->Time))
		{
			const FKeyHandle KeyHandle = CrossCurve->FindKey(InCrossCurveTime);
			if (KeyHandle != FKeyHandle::Invalid())
			{
				CrossCurve->SetKeyValue(KeyHandle, Iter->Value);
			}
		}
	}
}

FName FBaseLensTable::GetFriendlyPointName(ELensDataCategory InCategory)
{
	switch (InCategory)
	{
		case ELensDataCategory::Zoom: return TEXT("Focal Length");
		case ELensDataCategory::Distortion: return TEXT("Distortion Parameters");
		case ELensDataCategory::ImageCenter: return TEXT("Image Center");
		case ELensDataCategory::STMap: return TEXT("ST Map");
		case ELensDataCategory::NodalOffset: return TEXT("Nodal Offset");
	}

	return TEXT("");
}

void FBaseLensTable::ForEachFocusPoint(FFocusPointCallback InCallback, const float InFocus,float InputTolerance) const
{
	ForEachPoint([this, InCallback, InFocus, InputTolerance](const FBaseFocusPoint& InFocusPoint)
	{
		if (!FMath::IsNearlyEqual(InFocusPoint.GetFocus(), InFocus, InputTolerance))
		{
			return;
		}

		InCallback(InFocusPoint);
	});
}

void FBaseLensTable::ForEachLinkedFocusPoint(FLinkedFocusPointCallback InCallback, const float InFocus, float InputTolerance) const
{
	if (!ensure(LensFile.IsValid()))
	{
		return;
	}
	
	const TMap<ELensDataCategory, FLinkPointMetadata> LinkedCategories = GetLinkedCategories();
	for (const TPair<ELensDataCategory, FLinkPointMetadata>& LinkedCategoryPair : LinkedCategories)
	{
		const FBaseLensTable* const LinkDataTable = LensFile->GetDataTable(LinkedCategoryPair.Key);
		if (!ensure(LinkDataTable))
		{
			return;
		}
		
		LinkDataTable->ForEachPoint([this, InCallback, InFocus, InputTolerance, LinkedCategoryPair](const FBaseFocusPoint& InFocusPoint)
		{
			if (!FMath::IsNearlyEqual(InFocusPoint.GetFocus(), InFocus, InputTolerance))
			{
				return;
			}

			InCallback(InFocusPoint, LinkedCategoryPair.Key, LinkedCategoryPair.Value);
		});
	}
}

bool FBaseLensTable::HasLinkedFocusValues(const float InFocus, float InputTolerance) const
{
	if (!ensure(LensFile.IsValid()))
	{
		return false;
	}
	
	const TMap<ELensDataCategory, FLinkPointMetadata> LinkedCategories = GetLinkedCategories();
	for (const TPair<ELensDataCategory, FLinkPointMetadata>& LinkedCategoryPair : LinkedCategories)
	{
		const FBaseLensTable* const LinkDataTable = LensFile->GetDataTable(LinkedCategoryPair.Key);
		if (!ensure(LinkDataTable))
		{
			return false;
		}

		if (LinkDataTable->DoesFocusPointExists(InFocus, InputTolerance))
		{
			return true;
		}
	}

	return false;
}

bool FBaseLensTable::HasLinkedZoomValues(const float InFocus, const float InZoomPoint, float InputTolerance) const
{
	if (!ensure(LensFile.IsValid()))
	{
		return false;
	}
	
	const TMap<ELensDataCategory, FLinkPointMetadata> LinkedCategories = GetLinkedCategories();
	for (const TPair<ELensDataCategory, FLinkPointMetadata>& LinkedCategoryPair : LinkedCategories)
	{
		const FBaseLensTable* const LinkDataTable = LensFile->GetDataTable(LinkedCategoryPair.Key);
		if (!ensure(LinkDataTable))
		{
			return false;
		}
	
		if (LinkDataTable->DoesZoomPointExists(InFocus, InZoomPoint, InputTolerance))
		{
			return true;
		}
	}

	return false;
}

bool FBaseLensTable::IsFocusBetweenNeighbor(const float InFocusPoint, const float InFocusValueToEvaluate) const
{
	const int32 PointNum = GetFocusPointNum();

	// Return true if there is no neighbor and only one focus point
	if (PointNum == 1)
	{
		return true;
	}

	TOptional<float> MinValue;
	TOptional<float> MaxValue;

	// Loop through all table points
	for (int32 PointIndex = 0; PointIndex < PointNum; ++PointIndex)
	{
		if (const FBaseFocusPoint* const FocusPoint = GetBaseFocusPoint(PointIndex))
		{
			// check if the given point is same as point from loop
			if (FMath::IsNearlyEqual(FocusPoint->GetFocus(), InFocusPoint))
			{
				// Get neighbor point indexes
				const int32 MaxIndex = (PointIndex + 1);
				const int32 MinIndex = (PointIndex - 1);

				// Set min from index - 1 value
				if (MinIndex >= 0 && MinIndex < PointNum)
				{
					MinValue = GetBaseFocusPoint(MinIndex)->GetFocus();
				}
				// If min index not valid set the min value from current loop focus point
				else
				{
					MinValue = FocusPoint->GetFocus();
				}

				// Set max from index + 1 value
				if (MaxIndex < PointNum && PointNum >= 0)
				{
					MaxValue = GetBaseFocusPoint(MaxIndex)->GetFocus();
				}
				// If max index not valid set the max value from current loop focus point
				else
				{
					MaxValue = FocusPoint->GetFocus();
				}

				// Stop executing, if given point is same as point from loop
				break;
			}
		}
	}

	// If min or max not set or then are equal return false
	if (!MinValue.IsSet() || !MaxValue.IsSet() || FMath::IsNearlyEqual(MinValue.GetValue(), MaxValue.GetValue()))
	{
		return false;
	}

	// return true if evaluate value fit into the neighbor range
	if ((MinValue.GetValue() < InFocusValueToEvaluate || FMath::IsNearlyEqual(MinValue.GetValue(), InFocusValueToEvaluate)) &&
		(MaxValue.GetValue() > InFocusValueToEvaluate || FMath::IsNearlyEqual(MaxValue.GetValue(), InFocusValueToEvaluate)))
	{
		return true;
	}
	
	return false;
}

