// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tables/DistortionParametersTable.h"

#include "LensFile.h"
#include "LensTableUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DistortionParametersTable)

int32 FDistortionFocusPoint::GetNumPoints() const
{
	return MapBlendingCurve.GetNumKeys();
}

float FDistortionFocusPoint::GetZoom(int32 Index) const
{
	return MapBlendingCurve.Keys[Index].Time;
}

bool FDistortionFocusPoint::GetPoint(float InZoom, FDistortionInfo& OutData, float InputTolerance) const
{
	const FKeyHandle Handle = MapBlendingCurve.FindKey(InZoom, InputTolerance);
	if(Handle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = MapBlendingCurve.GetIndexSafe(Handle);
		check(ZoomPoints.IsValidIndex(PointIndex));

		OutData = ZoomPoints[PointIndex].DistortionInfo;

		return true;
	}

	return false;
}

bool FDistortionFocusPoint::AddPoint(float InZoom, const FDistortionInfo& InData, float InputTolerance, bool /*bIsCalibrationPoint*/)
{
	if (SetPoint(InZoom, InData, InputTolerance))
	{
		return true;
	}

	//Add new zoom point
	const FKeyHandle NewKeyHandle = MapBlendingCurve.AddKey(InZoom, InZoom);
	MapBlendingCurve.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
	MapBlendingCurve.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);

	//Insert point at the same index as the curve key
	const int32 KeyIndex = MapBlendingCurve.GetIndexSafe(NewKeyHandle);
	FDistortionZoomPoint NewZoomPoint;
	NewZoomPoint.Zoom = InZoom;
	NewZoomPoint.DistortionInfo = InData;
	ZoomPoints.Insert(MoveTemp(NewZoomPoint), KeyIndex);

	// The function return true all the time
	return true;
}

bool FDistortionFocusPoint::SetPoint(float InZoom, const FDistortionInfo& InData, float InputTolerance)
{
	const FKeyHandle Handle = MapBlendingCurve.FindKey(InZoom, InputTolerance);
	if(Handle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = MapBlendingCurve.GetIndexSafe(Handle);
		check(ZoomPoints.IsValidIndex(PointIndex));

		//No need to update map curve since x == y
		ZoomPoints[PointIndex].DistortionInfo = InData;

		return true;
	}

	return false;
}

void FDistortionFocusPoint::RemovePoint(float InZoomValue)
{
	const int32 FoundIndex = ZoomPoints.IndexOfByPredicate([InZoomValue](const FDistortionZoomPoint& Point) { return FMath::IsNearlyEqual(Point.Zoom, InZoomValue); });
	if(FoundIndex != INDEX_NONE)
	{
		ZoomPoints.RemoveAt(FoundIndex);
	}

	const FKeyHandle KeyHandle = MapBlendingCurve.FindKey(InZoomValue);
	if(KeyHandle != FKeyHandle::Invalid())
	{
		MapBlendingCurve.DeleteKey(KeyHandle);
	}
}

bool FDistortionFocusPoint::IsEmpty() const
{
	return MapBlendingCurve.IsEmpty();
}

const FRichCurve* FDistortionFocusPoint::GetCurveForParameter(int32 InParameterIndex) const
{
	if (InParameterIndex != FDistortionTable::FParameters::Aggregate)
	{
		return nullptr;
	}

	return &MapBlendingCurve;
}

void FDistortionFocusPoint::SetParameterValue(int32 InZoomIndex, float InZoomValue, int32 InParameterIndex, float InParameterValue)
{
	//We can't move keys on the time axis so our indices should match
	if (ZoomPoints.IsValidIndex(InZoomIndex))
	{
		if (ensure(FMath::IsNearlyEqual(ZoomPoints[InZoomIndex].Zoom, InZoomValue)))
		{
			ZoomPoints[InZoomIndex].DistortionInfo.Parameters[InParameterIndex] = InParameterValue;
		}
	}
}

void FDistortionFocusCurve::AddPoint(float InFocus, const FDistortionInfo& InData, float InputTolerance)
{
	AddPointToCurve(MapBlendingCurve, InFocus, InFocus, InputTolerance);
}

void FDistortionFocusCurve::SetPoint(float InFocus, const FDistortionInfo& InData, float InputTolerance)
{
	// No need to update the curve since curve time = curve value
}

void FDistortionFocusCurve::RemovePoint(float InFocus, float InputTolerance)
{
	DeletePointFromCurve(MapBlendingCurve, InFocus, InputTolerance);
}

void FDistortionFocusCurve::ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	ChangeFocusInCurve(MapBlendingCurve, InExistingFocus, InNewFocus, InputTolerance);
}

void FDistortionFocusCurve::MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	MergeFocusInCurve(MapBlendingCurve, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
}

bool FDistortionFocusCurve::IsEmpty() const
{
	return !MapBlendingCurve.GetNumKeys();
}

const FRichCurve* FDistortionFocusCurve::GetCurveForParameter(int32 InParameterIndex) const
{
	if (InParameterIndex != FDistortionTable::FParameters::Aggregate)
	{
		return nullptr;
	}

	return &MapBlendingCurve;
}

int32 FDistortionTable::GetTotalPointNum() const
{
	return LensDataTableUtils::GetTotalPointNum(FocusPoints);
}

UScriptStruct* FDistortionTable::GetScriptStruct() const
{
	return StaticStruct();
}

bool FDistortionTable::BuildParameterCurveAtFocus(float InFocus, int32 ParameterIndex, FRichCurve& OutCurve) const
{
	if (const FDistortionFocusPoint* ThisFocusPoints = GetFocusPoint(InFocus))
	{
		//Go over each zoom points
		for (const FDistortionZoomPoint& ZoomPoint : ThisFocusPoints->ZoomPoints)
		{
			if (ZoomPoint.DistortionInfo.Parameters.IsValidIndex(ParameterIndex))
			{
				const FKeyHandle Handle = OutCurve.AddKey(ZoomPoint.Zoom, ZoomPoint.DistortionInfo.Parameters[ParameterIndex]);
				OutCurve.SetKeyInterpMode(Handle, RCIM_Linear);
			}
			else //Defaults to map blending
			{
				OutCurve = ThisFocusPoints->MapBlendingCurve;
				return true;
			}
		}

		return true;
	}

	return false;
}

bool FDistortionTable::BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (InParameterIndex == FParameters::Aggregate)
	{
		const FDistortionFocusCurve* Curve = GetFocusCurve(InZoom);
		if (!Curve)
		{
			return false;
		}

		OutCurve = Curve->MapBlendingCurve;
		return true;
	}
	
	for (const FDistortionFocusPoint& FocusPoint : FocusPoints)
	{
		FDistortionInfo ZoomPoint;
		if (FocusPoint.GetPoint(InZoom, ZoomPoint))
		{
			float Value = 0.0;
			if (!ZoomPoint.Parameters.IsValidIndex(InParameterIndex))
			{
				return false;
			}
			
			Value = ZoomPoint.Parameters[InParameterIndex];
			
			const FKeyHandle NewKeyHandle = OutCurve.AddKey(FocusPoint.Focus, Value);
			FRichCurveKey& NewKey = OutCurve.GetKey(NewKeyHandle);
			NewKey.TangentMode = ERichCurveTangentMode::RCTM_None;
			NewKey.InterpMode = ERichCurveInterpMode::RCIM_Linear;
		}
	}

	return true;
}

void FDistortionTable::SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (FDistortionFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			const FKeyHandle Handle = InKeys[Index];
			const int32 KeyIndex = InSourceCurve.GetIndexSafe(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				//We can't move keys on the time axis so our indices should match
				const FRichCurveKey& Key = InSourceCurve.GetKey(Handle);

				if (InParameterIndex == FParameters::Aggregate)
				{
					FocusPoint->MapBlendingCurve.Keys[KeyIndex] = Key;
				}
				else
				{
					FocusPoint->SetParameterValue(KeyIndex, Key.Time, InParameterIndex, Key.Value);
				}
			}
		}
	}
}

void FDistortionTable::SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	// The aggregate curve's attributes can be changed, but not its key values, so simply copy the key attributes from the source curve
	// to the corresponding focus curve
	if (InParameterIndex == FParameters::Aggregate)
	{
		FDistortionFocusCurve* Curve = GetFocusCurve(InZoom);
		if (!Curve)
		{
			return;
		}

		CopyCurveKeys(InSourceCurve, Curve->MapBlendingCurve, InKeys);
		return;
	}

	// For every other parameter, only the curve's key values can be changed, so iterate over all keys to find the
	// appropriate focus/zoom point and update its value to match the curve's key values
	for (const FKeyHandle& KeyHandle : InKeys)
	{
		const FRichCurveKey& Key = InSourceCurve.GetKey(KeyHandle);
		if (FDistortionFocusPoint* FocusPoint = GetFocusPoint(Key.Time))
		{
			FDistortionInfo ZoomPoint;
			if (!FocusPoint->GetPoint(InZoom, ZoomPoint))
			{
				continue;
			}

			if (!ZoomPoint.Parameters.IsValidIndex(InParameterIndex))
			{
				return;
			}
		
			ZoomPoint.Parameters[InParameterIndex] = Key.Value;
			FocusPoint->SetPoint(InZoom, ZoomPoint);
		}
	}
}

bool FDistortionTable::CanEditCurveKeyPositions(int32 InParameterIndex) const
{
	return InParameterIndex != FParameters::Aggregate;
}

bool FDistortionTable::CanEditCurveKeyAttributes(int32 InParameterIndex) const
{
	return InParameterIndex == FParameters::Aggregate;
}

FText FDistortionTable::GetParameterValueLabel(int32 InParameterIndex) const
{
	if (InParameterIndex != FParameters::Aggregate)
	{
		return NSLOCTEXT("FDistortionTable", "ParameterValueLabel", "(unitless)");
	}
	
	return FText();
}

const FDistortionFocusPoint* FDistortionTable::GetFocusPoint(float InFocus, float InputTolerance) const
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FDistortionFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

FDistortionFocusPoint* FDistortionTable::GetFocusPoint(float InFocus, float InputTolerance)
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FDistortionFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

const FDistortionFocusCurve* FDistortionTable::GetFocusCurve(float InZoom, float InputTolerance) const
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FDistortionFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

FDistortionFocusCurve* FDistortionTable::GetFocusCurve(float InZoom, float InputTolerance)
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FDistortionFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

void FDistortionTable::ForEachPoint(FFocusPointCallback InCallback) const
{
	for (const FDistortionFocusPoint& Point : FocusPoints)
	{
		InCallback(Point);
	}
}

TConstArrayView<FDistortionFocusPoint> FDistortionTable::GetFocusPoints() const
{
	return FocusPoints;
}

TArray<FDistortionFocusPoint>& FDistortionTable::GetFocusPoints()
{
	return FocusPoints;
}

TConstArrayView<FDistortionFocusCurve> FDistortionTable::GetFocusCurves() const
{
	return FocusCurves;
}

TArray<FDistortionFocusCurve>& FDistortionTable::GetFocusCurves()
{
	return FocusCurves;
}

void FDistortionTable::RemoveFocusPoint(float InFocus)
{
	LensDataTableUtils::RemoveFocusPoint(FocusPoints, InFocus);
	LensDataTableUtils::RemoveFocusFromFocusCurves(FocusCurves, InFocus);
}

bool FDistortionTable::HasFocusPoint(float InFocus, float InputTolerance) const
{
	return DoesFocusPointExists(InFocus, InputTolerance);
}

void FDistortionTable::ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	LensDataTableUtils::ChangeFocusPoint(FocusPoints, InExistingFocus, InNewFocus, InputTolerance);
	LensDataTableUtils::ChangeFocusInFocusCurves(FocusCurves, InExistingFocus, InNewFocus, InputTolerance);
}

void FDistortionTable::MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance)
{
	LensDataTableUtils::MergeFocusPoint(FocusPoints, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
	LensDataTableUtils::MergeFocusInFocusCurves(FocusCurves, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
}

void FDistortionTable::RemoveZoomPoint(float InFocus, float InZoom)
{
	LensDataTableUtils::RemoveZoomPoint(FocusPoints, InFocus, InZoom);
	LensDataTableUtils::RemoveZoomFromFocusCurves(FocusCurves, InFocus, InZoom);
}

bool FDistortionTable::HasZoomPoint(float InFocus, float InZoom, float InputTolerance)
{
	return DoesZoomPointExists(InFocus, InZoom, InputTolerance);
}

void FDistortionTable::ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance)
{
	LensDataTableUtils::ChangeZoomPoint(FocusPoints, InFocus, InExistingZoom, InNewZoom, InputTolerance);

	FDistortionInfo Data;
	if (!GetPoint(InFocus, InNewZoom, Data, InputTolerance))
	{
		return;
	}

	LensDataTableUtils::ChangeZoomInFocusCurves(FocusCurves, InFocus, InExistingZoom, InNewZoom, Data, InputTolerance);
}

bool FDistortionTable::DoesFocusPointExists(float InFocus, float InputTolerance) const
{
	if (GetFocusPoint(InFocus, InputTolerance) != nullptr)
	{
		return true;
	}

	return false;
}

bool FDistortionTable::DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance) const
{
	FDistortionInfo DistortionInfo;
	if (GetPoint(InFocus, InZoom, DistortionInfo, InputTolerance))
	{
		return true;
	}

	return false;
}

const FBaseFocusPoint* FDistortionTable::GetBaseFocusPoint(int32 InIndex) const
{
	if (FocusPoints.IsValidIndex(InIndex))
	{
		return &FocusPoints[InIndex];
	}

	return nullptr;
}

TMap<ELensDataCategory, FLinkPointMetadata> FDistortionTable::GetLinkedCategories() const
{
	static TMap<ELensDataCategory, FLinkPointMetadata> LinkedToCategories =
	{
		{ELensDataCategory::Zoom, {true}},
		{ELensDataCategory::ImageCenter, {true}},
		{ELensDataCategory::NodalOffset, {false}},
	};
	return LinkedToCategories;
}

bool FDistortionTable::AddPoint(float InFocus, float InZoom, const FDistortionInfo& InData, float InputTolerance, bool bIsCalibrationPoint)
{
	if (!LensDataTableUtils::AddPoint(FocusPoints, InFocus, InZoom, InData, InputTolerance, bIsCalibrationPoint))
	{
		return false;
	}

	LensDataTableUtils::AddPointToFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

bool FDistortionTable::GetPoint(const float InFocus, const float InZoom, FDistortionInfo& OutData, float InputTolerance) const
{
	if (const FDistortionFocusPoint* DistortionFocusPoint = GetFocusPoint(InFocus, InputTolerance))
	{
		FDistortionInfo DistortionInfo;
		if (DistortionFocusPoint->GetPoint(InZoom, DistortionInfo, InputTolerance))
		{
			// Copy struct to outer
			OutData = DistortionInfo;
			return true;
		}
	}
	
	return false;
}

bool FDistortionTable::SetPoint(float InFocus, float InZoom, const FDistortionInfo& InData, float InputTolerance)
{
	if (!LensDataTableUtils::SetPoint(*this, InFocus, InZoom, InData, InputTolerance))
	{
		return false;
	}

	LensDataTableUtils::SetPointInFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

void FDistortionTable::BuildFocusCurves()
{
	// Ensure that the focus curves are empty before building them from the table data
	FocusCurves.Empty();
	LensDataTableUtils::BuildFocusCurves(FocusPoints, FocusCurves);
}



