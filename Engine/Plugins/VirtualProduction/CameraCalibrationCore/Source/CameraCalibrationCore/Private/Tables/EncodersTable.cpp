// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tables/EncodersTable.h"
#include "CameraCalibrationCoreLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EncodersTable)

int32 FEncodersTable::GetNumFocusPoints() const
{
	return Focus.GetNumKeys();
}

float FEncodersTable::GetFocusInput(int32 Index) const
{
	if (!ensure(Focus.Keys.IsValidIndex(Index)))
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Invalid focus keys index: %d", Index);
		return 0.f;
	}
	
	return Focus.Keys[Index].Time;
}

float FEncodersTable::GetFocusValue(int32 Index) const
{
	if (!ensure(Focus.Keys.IsValidIndex(Index)))
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Invalid focus keys index: %d", Index);
		return 0.f;
	}
	
	return Focus.Keys[Index].Value;
}

bool FEncodersTable::RemoveFocusPoint(float InRawFocus)
{
	const FKeyHandle FoundKey = Focus.FindKey(InRawFocus);
	if(FoundKey != FKeyHandle::Invalid())
	{
		Focus.DeleteKey(FoundKey);
		return true;
	}

	return false;
}

int32 FEncodersTable::GetNumIrisPoints() const
{
	return Iris.GetNumKeys();
}

float FEncodersTable::GetIrisInput(int32 Index) const
{
	if (!ensure(Iris.Keys.IsValidIndex(Index)))
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Invalid iris keys index: %d", Index);
		return 0.f;
	}

	return Iris.Keys[Index].Time;
}

float FEncodersTable::GetIrisValue(int32 Index) const
{
	if (!ensure(Iris.Keys.IsValidIndex(Index)))
	{
		UE_LOGF(LogCameraCalibrationCore, Error, "Invalid iris keys index: %d", Index);
		return 0.f;
	}

	return Iris.Keys[Index].Value;
}

bool FEncodersTable::RemoveIrisPoint(float InRawIris)
{
	const FKeyHandle FoundKey = Iris.FindKey(InRawIris);
	if(FoundKey != FKeyHandle::Invalid())
	{
		Iris.DeleteKey(FoundKey);
		return true;
	}

	return false;
}

void FEncodersTable::ClearAll()
{
	Focus.Reset();
	Iris.Reset();
}
