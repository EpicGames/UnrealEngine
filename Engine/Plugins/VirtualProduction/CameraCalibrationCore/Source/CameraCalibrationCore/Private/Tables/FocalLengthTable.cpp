// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tables/FocalLengthTable.h"

#include "LensFile.h"
#include "LensTableUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FocalLengthTable)

int32 FFocalLengthFocusPoint::GetNumPoints() const
{
	return Fx.GetNumKeys();
}

float FFocalLengthFocusPoint::GetZoom(int32 Index) const
{
	return Fx.Keys[Index].Time;
}

bool FFocalLengthFocusPoint::GetPoint(float InZoom, FFocalLengthInfo& OutData, float InputTolerance) const
{
	const FKeyHandle FxHandle = Fx.FindKey(InZoom, InputTolerance);
	if(FxHandle != FKeyHandle::Invalid())
	{
		const FKeyHandle FyHandle = Fy.FindKey(InZoom, InputTolerance);
		const int32 PointIndex = Fx.GetIndexSafe(FxHandle);
		check(FyHandle != FKeyHandle::Invalid() && ZoomPoints.IsValidIndex(PointIndex))

		OutData.FxFy.X = Fx.GetKeyValue(FxHandle);
		OutData.FxFy.Y = Fy.GetKeyValue(FyHandle);

		return true;
	}

	return false;
}

bool FFocalLengthFocusPoint::AddPoint(float InZoom, const FFocalLengthInfo& InData, float InputTolerance, bool bIsCalibrationPoint)
{
	if (SetPoint(InZoom, InData, InputTolerance))
	{
		return true;
	}
	
	//Add new zoom point
	const FKeyHandle NewKeyHandle = Fx.AddKey(InZoom, InData.FxFy.X);
	Fx.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
	Fx.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);

	Fy.AddKey(InZoom, InData.FxFy.Y, false, NewKeyHandle);
	Fy.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
	Fy.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);
	
	const int32 KeyIndex = Fx.GetIndexSafe(NewKeyHandle);
	FFocalLengthZoomPoint NewFocalLengthPoint;
	NewFocalLengthPoint.Zoom = InZoom;
	NewFocalLengthPoint.bIsCalibrationPoint = bIsCalibrationPoint;
	NewFocalLengthPoint.FocalLengthInfo.FxFy = InData.FxFy;
	ZoomPoints.Insert(MoveTemp(NewFocalLengthPoint), KeyIndex);

	// The function return true all the time
	return true;
}

bool FFocalLengthFocusPoint::SetPoint(float InZoom, const FFocalLengthInfo& InData, float InputTolerance)
{
	const FKeyHandle FxHandle = Fx.FindKey(InZoom, InputTolerance);
	if(FxHandle != FKeyHandle::Invalid())
	{
		const FKeyHandle FyHandle = Fy.FindKey(InZoom, InputTolerance);
		const int32 PointIndex = Fx.GetIndexSafe(FxHandle);
		check(FyHandle != FKeyHandle::Invalid() && ZoomPoints.IsValidIndex(PointIndex))
	
		Fx.SetKeyValue(FxHandle, InData.FxFy.X);
		Fy.SetKeyValue(FyHandle, InData.FxFy.Y);
		ZoomPoints[PointIndex].FocalLengthInfo = InData;

		return true;
	}

	return false;
}

bool FFocalLengthFocusPoint::IsCalibrationPoint(float InZoom, float InputTolerance)
{
	const FKeyHandle FxHandle = Fx.FindKey(InZoom, InputTolerance);
	if (FxHandle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = Fx.GetIndexSafe(FxHandle);
		check(ZoomPoints.IsValidIndex(PointIndex))

		return ZoomPoints[PointIndex].bIsCalibrationPoint;
	}

	return false;
}

bool FFocalLengthFocusPoint::GetValue(int32 Index, FFocalLengthInfo& OutData) const
{
	if(ZoomPoints.IsValidIndex(Index))
	{
		OutData = ZoomPoints[Index].FocalLengthInfo;

		return true;
	}

	return false;
}

void FFocalLengthFocusPoint::RemovePoint(float InZoomValue)
{
	const int32 FoundIndex = ZoomPoints.IndexOfByPredicate([InZoomValue](const FFocalLengthZoomPoint& Point) { return FMath::IsNearlyEqual(Point.Zoom, InZoomValue); });
	if(FoundIndex != INDEX_NONE)
	{
		ZoomPoints.RemoveAt(FoundIndex);
	}

	const FKeyHandle FxHandle = Fx.FindKey(InZoomValue);
	if(FxHandle != FKeyHandle::Invalid())
	{
		Fx.DeleteKey(FxHandle);
	}

	const FKeyHandle FyHandle = Fy.FindKey(InZoomValue);
	if (FyHandle != FKeyHandle::Invalid())
	{
		Fy.DeleteKey(FyHandle);
	}
}

bool FFocalLengthFocusPoint::IsEmpty() const
{
	return Fx.IsEmpty();
}

const FRichCurve* FFocalLengthFocusPoint::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FFocalLengthTable::FParameters::IsValid(InParameterIndex))
	{
		if (InParameterIndex == FFocalLengthTable::FParameters::Fx)
		{
			return &Fx;
		}

		if (InParameterIndex == FFocalLengthTable::FParameters::Fy)
		{
			return &Fy;
		}
	}

	return nullptr;
}

FRichCurve* FFocalLengthFocusPoint::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FFocalLengthFocusPoint*>(this)->GetCurveForParameter(InParameterIndex));
}

void FFocalLengthFocusCurve::AddPoint(float InFocus, const FFocalLengthInfo& InData, float InputTolerance)
{
	const FKeyHandle KeyHandle = AddPointToCurve(Fx, InFocus, InData.FxFy.X, InputTolerance);
	AddPointToCurve(Fy, InFocus, InData.FxFy.Y, InputTolerance, KeyHandle);
}

void FFocalLengthFocusCurve::SetPoint(float InFocus, const FFocalLengthInfo& InData, float InputTolerance)
{
	SetPointInCurve(Fx, InFocus, InData.FxFy.X, InputTolerance);
	SetPointInCurve(Fy, InFocus, InData.FxFy.Y, InputTolerance);
}

void FFocalLengthFocusCurve::RemovePoint(float InFocus, float InputTolerance)
{
	DeletePointFromCurve(Fx, InFocus, InputTolerance);
	DeletePointFromCurve(Fy, InFocus, InputTolerance);
}

void FFocalLengthFocusCurve::ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	ChangeFocusInCurve(Fx, InExistingFocus, InNewFocus, InputTolerance);
	ChangeFocusInCurve(Fy, InExistingFocus, InNewFocus, InputTolerance);
}

void FFocalLengthFocusCurve::MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	MergeFocusInCurve(Fx, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
	MergeFocusInCurve(Fy, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
}

bool FFocalLengthFocusCurve::IsEmpty() const
{
	return !Fx.GetNumKeys() && !Fy.GetNumKeys();
}

const FRichCurve* FFocalLengthFocusCurve::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FFocalLengthTable::FParameters::IsValid(InParameterIndex))
	{
		if (InParameterIndex == FFocalLengthTable::FParameters::Fx)
		{
			return &Fx;
		}

		if (InParameterIndex == FFocalLengthTable::FParameters::Fy)
		{
			return &Fy;
		}
	}

	return nullptr;
}

FRichCurve* FFocalLengthFocusCurve::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FFocalLengthFocusCurve*>(this)->GetCurveForParameter(InParameterIndex));
}

bool FFocalLengthTable::DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance) const
{
	FFocalLengthInfo FocalLengthInfo;
	if (GetPoint(InFocus, InZoom, FocalLengthInfo, InputTolerance))
	{
		return true;
	}

	return false;
}

const FBaseFocusPoint* FFocalLengthTable::GetBaseFocusPoint(int32 InIndex) const
{
	if (FocusPoints.IsValidIndex(InIndex))
	{
		return &FocusPoints[InIndex];
	}

	return nullptr;
}


TMap<ELensDataCategory, FLinkPointMetadata> FFocalLengthTable::GetLinkedCategories() const
{
	static TMap<ELensDataCategory, FLinkPointMetadata> LinkedToCategories =
	{
		{ELensDataCategory::Distortion, {true}},
		{ELensDataCategory::ImageCenter, {true}},
		{ELensDataCategory::STMap, {true}},
		{ELensDataCategory::NodalOffset, {false}},
	};
	return LinkedToCategories;
}

int32 FFocalLengthTable::GetTotalPointNum() const
{
	return LensDataTableUtils::GetTotalPointNum(FocusPoints);
}

UScriptStruct* FFocalLengthTable::GetScriptStruct() const
{
	return StaticStruct();
}

bool FFocalLengthTable::BuildParameterCurveAtFocus(float InFocus, int32 ParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValidOrAggregate(ParameterIndex))
	{
		return false;
	}
	
	const FFocalLengthFocusPoint* FocusPoint = GetFocusPoint(InFocus);
	if (!FocusPoint)
	{
		return false;
	}

	if (ParameterIndex == FParameters::Aggregate)
	{
		// The aggregate curve is just the x curve scaled by the lens file's sensor
		const FRichCurve& ActiveCurve = FocusPoint->Fx;
		auto HandleIter = ActiveCurve.GetKeyHandleIterator();
		for (const FRichCurveKey& Key : ActiveCurve.GetConstRefOfKeys())
		{
			ULensFile* LensFilePtr = GetLensFile();
			const float Scale = LensFilePtr ? LensFilePtr->LensInfo.SensorDimensions.X : 1.0f;
			OutCurve.AddKey(Key.Time, Key.Value * Scale, false, *HandleIter);

			FRichCurveKey& NewKey = OutCurve.GetKey(*HandleIter);
			NewKey.TangentMode = Key.TangentMode;
			NewKey.InterpMode = Key.InterpMode;
			NewKey.ArriveTangent = Key.ArriveTangent * Scale;
			NewKey.LeaveTangent = Key.LeaveTangent * Scale;
			++HandleIter;
		}

		return true;
	}
	else if (ParameterIndex == FParameters::Fx)
	{
		OutCurve = FocusPoint->Fx;
		return true;
	}
	else if (ParameterIndex == FParameters::Fy)
	{
		OutCurve = FocusPoint->Fy;
		return true;
	}

	return false;
}

bool FFocalLengthTable::BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValidOrAggregate(InParameterIndex))
	{
		return false;
	}

	const FFocalLengthFocusCurve* FocusCurve = GetFocusCurve(InZoom);
	if (!FocusCurve)
	{
		return false;
	}

	if (InParameterIndex == FParameters::Aggregate)
	{
		// The aggregate curve is just the x curve scaled by the lens file's sensor
		const FRichCurve& ActiveCurve = FocusCurve->Fx;
		auto HandleIter = ActiveCurve.GetKeyHandleIterator();
		for (const FRichCurveKey& Key : ActiveCurve.GetConstRefOfKeys())
		{
			ULensFile* LensFilePtr = GetLensFile();
			const float Scale = LensFilePtr ? LensFilePtr->LensInfo.SensorDimensions.X : 1.0f;
			OutCurve.AddKey(Key.Time, Key.Value * Scale, false, *HandleIter);

			FRichCurveKey& NewKey = OutCurve.GetKey(*HandleIter);
			NewKey.TangentMode = Key.TangentMode;
			NewKey.InterpMode = Key.InterpMode;
			NewKey.ArriveTangent = Key.ArriveTangent * Scale;
			NewKey.LeaveTangent = Key.LeaveTangent * Scale;
			++HandleIter;
		}

		return true;
	}
	else if (InParameterIndex == FParameters::Fx)
	{
		OutCurve = FocusCurve->Fx;
		return true;
	}
	else if (InParameterIndex == FParameters::Fy)
	{
		OutCurve = FocusCurve->Fy;
		return true;
	}

	return false;
}

void FFocalLengthTable::SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValidOrAggregate(InParameterIndex))
	{
		return;
	}
	
	FFocalLengthFocusPoint* FocusPoint = GetFocusPoint(InFocus);
	if (!FocusPoint)
	{
		return;
	}
	
	FRichCurve* ActiveCurve = nullptr;
	float Scale = 1.0f;
	int32 FxFyIndex = InParameterIndex;
	if (InParameterIndex == FParameters::Aggregate)
	{
		ActiveCurve = &FocusPoint->Fx;
		
		if (ULensFile* LensFilePtr = GetLensFile())
		{
			Scale = 1.0f / LensFilePtr->LensInfo.SensorDimensions.X;
		}
		
		FxFyIndex = 0; //mm focal length curve changes Fx
	}
	else
	{
		ActiveCurve = FocusPoint->GetCurveForParameter(InParameterIndex);
	}

	if (!ActiveCurve)
	{
		return;
	}

	for (const FKeyHandle& KeyHandle : InKeys)
	{
		const int32 KeyIndex = InSourceCurve.GetIndexSafe(KeyHandle);
		if (KeyIndex != INDEX_NONE)
		{
			if(ensure(ActiveCurve->Keys.IsValidIndex(KeyIndex) && FocusPoint->ZoomPoints.IsValidIndex(KeyIndex)))
			{
				const FRichCurveKey& SourceKey = InSourceCurve.GetKey(KeyHandle);
				FRichCurveKey& DestinationKey = ActiveCurve->Keys[KeyIndex];
				
				DestinationKey.Value = SourceKey.Value * Scale;
				DestinationKey.InterpMode = SourceKey.InterpMode;
				DestinationKey.ArriveTangent = SourceKey.ArriveTangent * Scale;
				DestinationKey.LeaveTangent = SourceKey.LeaveTangent * Scale;
				DestinationKey.TangentMode = SourceKey.TangentMode;
				
				FocusPoint->ZoomPoints[KeyIndex].FocalLengthInfo.FxFy[FxFyIndex] = SourceKey.Value * Scale;
			}
		}
	}	

	ActiveCurve->AutoSetTangents();
	
	PropagateCurveValuesToCrossCurves(*ActiveCurve, InFocus, [this, InParameterIndex](float InZoom)->FRichCurve*
	{
		if (FFocalLengthFocusCurve* Curve = GetFocusCurve(InZoom))
		{
			return Curve->GetCurveForParameter(InParameterIndex);
		}

		return nullptr;
	});
}

void FFocalLengthTable::SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValidOrAggregate(InParameterIndex))
	{
		return;
	}
	
	FFocalLengthFocusCurve* FocusCurve = GetFocusCurve(InZoom);
	if (!FocusCurve)
	{
		return;
	}
	
	FRichCurve* ActiveCurve = nullptr;
	float Scale = 1.0f;
	int32 FxFyIndex = InParameterIndex;
	if (InParameterIndex == FParameters::Aggregate)
	{
		ActiveCurve = &FocusCurve->Fx;
		
		if (ULensFile* LensFilePtr = GetLensFile())
		{
			Scale = 1.0f / LensFilePtr->LensInfo.SensorDimensions.X;
		}
		
		FxFyIndex = 0; //mm focal length curve changes Fx
	}
	else
	{
		ActiveCurve = FocusCurve->GetCurveForParameter(InParameterIndex);
	}

	if (!ActiveCurve)
	{
		return;
	}

	for (const FKeyHandle& KeyHandle : InKeys)
	{
		const int32 KeyIndex = InSourceCurve.GetIndexSafe(KeyHandle);
		if (KeyIndex != INDEX_NONE)
		{
			if(ensure(ActiveCurve->Keys.IsValidIndex(KeyIndex)))
			{
				const FRichCurveKey& SourceKey = InSourceCurve.GetKey(KeyHandle);
				FRichCurveKey& DestinationKey = ActiveCurve->Keys[KeyIndex];
				
				DestinationKey.Value = SourceKey.Value * Scale;
				DestinationKey.InterpMode = SourceKey.InterpMode;
				DestinationKey.ArriveTangent = SourceKey.ArriveTangent * Scale;
				DestinationKey.LeaveTangent = SourceKey.LeaveTangent * Scale;
				DestinationKey.TangentMode = SourceKey.TangentMode;

				if (FFocalLengthFocusPoint* FocusPoint = GetFocusPoint(SourceKey.Time))
				{
					FFocalLengthInfo ZoomPoint;
					if (FocusPoint->GetPoint(InZoom, ZoomPoint))
					{
						ZoomPoint.FxFy[FxFyIndex] = InSourceCurve.GetKeyValue(KeyHandle) * Scale;
						FocusPoint->SetPoint(InZoom, ZoomPoint);
						
						if (InParameterIndex == FParameters::Fy)
						{
							FocusPoint->Fy.AutoSetTangents();
						}
						else
						{
							FocusPoint->Fx.AutoSetTangents();
						}
					}
				}
			}
		}
	}	

	ActiveCurve->AutoSetTangents();
}

TRange<double> FFocalLengthTable::GetCurveKeyPositionRange(int32 InParameterIndex) const
{
	TRange<double> Range = FBaseLensTable::GetCurveKeyPositionRange(InParameterIndex);
	
	if (!FParameters::IsValidOrAggregate(InParameterIndex))
	{
		return Range;
	}

	if (InParameterIndex == FParameters::Aggregate)
	{
		Range.SetLowerBoundValue(1.0);
	}
	else if (ULensFile* LensFilePtr = GetLensFile())
	{
		Range.SetLowerBoundValue(1.0 / LensFilePtr->LensInfo.SensorDimensions[InParameterIndex]);
	}

	return Range;
}

FText FFocalLengthTable::GetParameterValueLabel(int32 InParameterIndex) const
{
	if (!FParameters::IsValidOrAggregate(InParameterIndex))
	{
		return FText();
	}
	
	if (InParameterIndex == FParameters::Aggregate)
	{
		return NSLOCTEXT("FFocalLengthTable", "ParameterValueMMLabel", "(mm)");
	}
	
	return NSLOCTEXT("FFocalLengthTable", "ParameterValueNormalizedLabel", "(normalized)");
}

FText FFocalLengthTable::GetParameterValueUnitLabel(int32 InParameterIndex) const
{
	if (InParameterIndex == FParameters::Aggregate)
	{
		return NSLOCTEXT("FFocalLengthTable", "ParameterUnitLabel", "mm");
	}
	
	return FText();
}

const FFocalLengthFocusPoint* FFocalLengthTable::GetFocusPoint(float InFocus, float InputTolerance) const
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FFocalLengthFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

FFocalLengthFocusPoint* FFocalLengthTable::GetFocusPoint(float InFocus, float InputTolerance)
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FFocalLengthFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

const FFocalLengthFocusCurve* FFocalLengthTable::GetFocusCurve(float InZoom, float InputTolerance) const
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FFocalLengthFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

FFocalLengthFocusCurve* FFocalLengthTable::GetFocusCurve(float InZoom, float InputTolerance)
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FFocalLengthFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

TConstArrayView<FFocalLengthFocusPoint> FFocalLengthTable::GetFocusPoints() const
{
	return FocusPoints;
}

TArray<FFocalLengthFocusPoint>& FFocalLengthTable::GetFocusPoints()
{
	return FocusPoints;
}

TConstArrayView<FFocalLengthFocusCurve> FFocalLengthTable::GetFocusCurves() const
{
	return FocusCurves;
}

TArray<FFocalLengthFocusCurve>& FFocalLengthTable::GetFocusCurves()
{
	return FocusCurves;
}

void FFocalLengthTable::ForEachPoint(FFocusPointCallback InCallback) const
{
	for (const FFocalLengthFocusPoint& Point : FocusPoints)
	{
		InCallback(Point);
	}
}

void FFocalLengthTable::RemoveFocusPoint(float InFocus)
{
	LensDataTableUtils::RemoveFocusPoint(FocusPoints, InFocus);
	LensDataTableUtils::RemoveFocusFromFocusCurves(FocusCurves, InFocus);
}

bool FFocalLengthTable::HasFocusPoint(float InFocus,  float InputTolerance) const
{
	return DoesFocusPointExists(InFocus, InputTolerance);
}

void FFocalLengthTable::ChangeFocusPoint(float InExistingFocus, float InNewFocus,  float InputTolerance)
{
	LensDataTableUtils::ChangeFocusPoint(FocusPoints, InExistingFocus, InNewFocus, InputTolerance);
	LensDataTableUtils::ChangeFocusInFocusCurves(FocusCurves, InExistingFocus, InNewFocus, InputTolerance);
}

void FFocalLengthTable::MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints,  float InputTolerance)
{
	LensDataTableUtils::MergeFocusPoint(FocusPoints, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
	LensDataTableUtils::MergeFocusInFocusCurves(FocusCurves, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
}

void FFocalLengthTable::RemoveZoomPoint(float InFocus, float InZoom)
{
	LensDataTableUtils::RemoveZoomPoint(FocusPoints, InFocus, InZoom);
	LensDataTableUtils::RemoveZoomFromFocusCurves(FocusCurves, InFocus, InZoom);
}

bool FFocalLengthTable::HasZoomPoint(float InFocus, float InZoom,  float InputTolerance)
{
	return DoesZoomPointExists(InFocus, InZoom, InputTolerance);
}

void FFocalLengthTable::ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom,  float InputTolerance)
{
	LensDataTableUtils::ChangeZoomPoint(FocusPoints, InFocus, InExistingZoom, InNewZoom, InputTolerance);

	FFocalLengthInfo Data;
	if (!GetPoint(InFocus, InNewZoom, Data, InputTolerance))
	{
		return;
	}
	
	LensDataTableUtils::ChangeZoomInFocusCurves(FocusCurves, InFocus, InExistingZoom, InNewZoom, Data, InputTolerance);
}

bool FFocalLengthTable::DoesFocusPointExists(float InFocus, float InputTolerance) const
{
	if (GetFocusPoint(InFocus, InputTolerance) != nullptr)
	{
		return true;
	}

	return false;
}

bool FFocalLengthTable::AddPoint(float InFocus, float InZoom, const FFocalLengthInfo& InData, float InputTolerance, bool bIsCalibrationPoint)
{
	if (!LensDataTableUtils::AddPoint(FocusPoints, InFocus, InZoom, InData, InputTolerance, bIsCalibrationPoint))
	{
		return false;
	}

	LensDataTableUtils::AddPointToFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

bool FFocalLengthTable::GetPoint(const float InFocus, const float InZoom, FFocalLengthInfo& OutData, float InputTolerance) const
{
	if (const FFocalLengthFocusPoint* FocalLengthFocusPoint = GetFocusPoint(InFocus, InputTolerance))
	{
		if (FocalLengthFocusPoint->GetPoint(InZoom, OutData, InputTolerance))
		{
			return true;
		}
	}
	
	return false;
}

bool FFocalLengthTable::SetPoint(float InFocus, float InZoom, const FFocalLengthInfo& InData, float InputTolerance)
{
	if (!LensDataTableUtils::SetPoint(*this, InFocus, InZoom, InData, InputTolerance))
	{
		return false;
	}

	LensDataTableUtils::SetPointInFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

void FFocalLengthTable::BuildFocusCurves()
{
	// Ensure that the focus curves are empty before building them from the table data
	FocusCurves.Empty();
	LensDataTableUtils::BuildFocusCurves(FocusPoints, FocusCurves);
}
