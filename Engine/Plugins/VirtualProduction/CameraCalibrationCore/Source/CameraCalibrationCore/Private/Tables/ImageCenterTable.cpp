// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tables/ImageCenterTable.h"

#include "LensFile.h"
#include "LensTableUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageCenterTable)

int32 FImageCenterFocusPoint::GetNumPoints() const
{
	return Cx.GetNumKeys();
}

float FImageCenterFocusPoint::GetZoom(int32 Index) const
{
	return Cx.Keys[Index].Time;
}

bool FImageCenterFocusPoint::GetPoint(float InZoom, FImageCenterInfo& OutData, float InputTolerance) const
{
	const FKeyHandle CxHandle = Cx.FindKey(InZoom, InputTolerance);
	if (CxHandle != FKeyHandle::Invalid())
	{
		const FKeyHandle CyHandle = Cy.FindKey(InZoom, InputTolerance);
		check(CyHandle != FKeyHandle::Invalid());

		OutData.PrincipalPoint.X = Cx.GetKeyValue(CxHandle);
		OutData.PrincipalPoint.Y = Cy.GetKeyValue(CyHandle);

		return true;
	}

	
	return false;
}

bool FImageCenterFocusPoint::AddPoint(float InZoom, const FImageCenterInfo& InData, float InputTolerance, bool /*bIsCalibrationPoint*/)
{
	if (SetPoint(InZoom, InData, InputTolerance))
	{
		return true;
	}
	
	//Add new zoom point
	const FKeyHandle NewKeyHandle = Cx.AddKey(InZoom, InData.PrincipalPoint.X);
	Cx.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
	Cx.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);

	Cy.AddKey(InZoom, InData.PrincipalPoint.Y, false, NewKeyHandle);
	Cy.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
	Cy.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);

	// The function return true all the time
	return true;
}

bool FImageCenterFocusPoint::SetPoint(float InZoom, const FImageCenterInfo& InData, float InputTolerance)
{
	const FKeyHandle CxHandle = Cx.FindKey(InZoom, InputTolerance);
	if (CxHandle != FKeyHandle::Invalid())
	{
		const FKeyHandle CyHandle = Cy.FindKey(InZoom, InputTolerance);
		check(CyHandle != FKeyHandle::Invalid());
		Cx.SetKeyValue(CxHandle, InData.PrincipalPoint.X);
		Cy.SetKeyValue(CyHandle, InData.PrincipalPoint.Y);

		return true;
	}

	return false;
}

void FImageCenterFocusPoint::RemovePoint(float InZoomValue)
{
	const FKeyHandle CxKeyHandle = Cx.FindKey(InZoomValue);
	if(CxKeyHandle != FKeyHandle::Invalid())
	{
		Cx.DeleteKey(CxKeyHandle);
	}

	const FKeyHandle CyKeyHandle = Cy.FindKey(InZoomValue);
	if (CyKeyHandle != FKeyHandle::Invalid())
	{
		Cy.DeleteKey(CyKeyHandle);
	}

}

bool FImageCenterFocusPoint::IsEmpty() const
{
	return Cx.IsEmpty();
}

const FRichCurve* FImageCenterFocusPoint::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FImageCenterTable::FParameters::IsValid(InParameterIndex))
	{
		if (InParameterIndex == FImageCenterTable::FParameters::Cx)
		{
			return &Cx;
		}

		if (InParameterIndex == FImageCenterTable::FParameters::Cy)
		{
			return &Cy;
		}
	}

	return nullptr;
}

FRichCurve* FImageCenterFocusPoint::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FImageCenterFocusPoint*>(this)->GetCurveForParameter(InParameterIndex));
}

void FImageCenterFocusCurve::AddPoint(float InFocus, const FImageCenterInfo& InData, float InputTolerance)
{
	const FKeyHandle KeyHandle = AddPointToCurve(Cx, InFocus, InData.PrincipalPoint.X, InputTolerance);
	AddPointToCurve(Cy, InFocus, InData.PrincipalPoint.Y, InputTolerance, KeyHandle);
}

void FImageCenterFocusCurve::SetPoint(float InFocus, const FImageCenterInfo& InData, float InputTolerance)
{
	SetPointInCurve(Cx, InFocus, InData.PrincipalPoint.X, InputTolerance);
	SetPointInCurve(Cy, InFocus, InData.PrincipalPoint.Y, InputTolerance);
}

void FImageCenterFocusCurve::RemovePoint(float InFocus, float InputTolerance)
{
	DeletePointFromCurve(Cx, InFocus, InputTolerance);
	DeletePointFromCurve(Cy, InFocus, InputTolerance);
}

void FImageCenterFocusCurve::ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	ChangeFocusInCurve(Cx, InExistingFocus, InNewFocus, InputTolerance);
	ChangeFocusInCurve(Cy, InExistingFocus, InNewFocus, InputTolerance);
}

void FImageCenterFocusCurve::MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	MergeFocusInCurve(Cx, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
	MergeFocusInCurve(Cy, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
}

bool FImageCenterFocusCurve::IsEmpty() const
{
	return !Cx.GetNumKeys() && !Cy.GetNumKeys();
}

const FRichCurve* FImageCenterFocusCurve::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FImageCenterTable::FParameters::IsValid(InParameterIndex))
	{
		if (InParameterIndex == FImageCenterTable::FParameters::Cx)
		{
			return &Cx;
		}

		if (InParameterIndex == FImageCenterTable::FParameters::Cy)
		{
			return &Cy;
		}
	}

	return nullptr;
}

FRichCurve* FImageCenterFocusCurve::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FImageCenterFocusCurve*>(this)->GetCurveForParameter(InParameterIndex));
}

void FImageCenterTable::ForEachPoint(FFocusPointCallback InCallback) const
{
	for (const FImageCenterFocusPoint& Point : FocusPoints)
	{
		InCallback(Point);
	}
}

bool FImageCenterTable::DoesFocusPointExists(float InFocus, float InputTolerance) const
{
	if (GetFocusPoint(InFocus, InputTolerance) != nullptr)
	{
		return true;
	}

	return false;
}

int32 FImageCenterTable::GetTotalPointNum() const
{
	return LensDataTableUtils::GetTotalPointNum(FocusPoints);
}

UScriptStruct* FImageCenterTable::GetScriptStruct() const
{
	return StaticStruct();
}

bool FImageCenterTable::BuildParameterCurveAtFocus(float InFocus, int32 ParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValid(ParameterIndex))
	{
		return false;
	}
	
	if (const FImageCenterFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		if (ParameterIndex == FParameters::Cx)
		{
			OutCurve = FocusPoint->Cx;
		}
		else
		{
			OutCurve = FocusPoint->Cy;
		}
		
		return true;
	}

	return false;
}

bool FImageCenterTable::BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValid(InParameterIndex))
	{
		return false;
	}

	if (const FImageCenterFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		if (InParameterIndex == FParameters::Cx)
		{
			OutCurve = FocusCurve->Cx;
		}
		else
		{
			OutCurve = FocusCurve->Cy;
		}

		return true;
	}

	return false;
}

void FImageCenterTable::SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValid(InParameterIndex))
	{
		return;
	}
	
	if (FImageCenterFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		CopyCurveKeys(InSourceCurve, *FocusPoint->GetCurveForParameter(InParameterIndex), InKeys);
		PropagateCurveValuesToCrossCurves(*FocusPoint->GetCurveForParameter(InParameterIndex), InFocus, [this, InParameterIndex](float InZoom)->FRichCurve*
		{
			if (FImageCenterFocusCurve* Curve = GetFocusCurve(InZoom))
			{
				return Curve->GetCurveForParameter(InParameterIndex);
			}

			return nullptr;
		});
	}
}

void FImageCenterTable::SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValid(InParameterIndex))
	{
		return;
	}
	
	if (FImageCenterFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		CopyCurveKeys(InSourceCurve, *FocusCurve->GetCurveForParameter(InParameterIndex), InKeys);
		PropagateCurveValuesToCrossCurves(*FocusCurve->GetCurveForParameter(InParameterIndex), InZoom, [this, InParameterIndex](float InFocus)->FRichCurve*
		{
			if (FImageCenterFocusPoint* FocusPoint = GetFocusPoint(InFocus))
			{
				return FocusPoint->GetCurveForParameter(InParameterIndex);
			}

			return nullptr;
		});
	}
}

FText FImageCenterTable::GetParameterValueLabel(int32 InParameterIndex) const
{
	return NSLOCTEXT("FImageCenterTable", "ParameterValueLabel", "(normalized)");
}

const FImageCenterFocusPoint* FImageCenterTable::GetFocusPoint(float InFocus, float InputTolerance) const
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FImageCenterFocusPoint& Points) { return FMath::IsNearlyEqual(Points.Focus, InFocus, InputTolerance); });
}

FImageCenterFocusPoint* FImageCenterTable::GetFocusPoint(float InFocus, float InputTolerance)
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FImageCenterFocusPoint& Points) { return FMath::IsNearlyEqual(Points.Focus, InFocus, InputTolerance); });
}

const FImageCenterFocusCurve* FImageCenterTable::GetFocusCurve(float InZoom, float InputTolerance) const
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FImageCenterFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

FImageCenterFocusCurve* FImageCenterTable::GetFocusCurve(float InZoom, float InputTolerance)
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FImageCenterFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

TConstArrayView<FImageCenterFocusPoint> FImageCenterTable::GetFocusPoints() const
{
	return FocusPoints;
}

TArray<FImageCenterFocusPoint>& FImageCenterTable::GetFocusPoints()
{
	return FocusPoints;
}

TConstArrayView<FImageCenterFocusCurve> FImageCenterTable::GetFocusCurves() const
{
	return FocusCurves;
}

TArray<FImageCenterFocusCurve>& FImageCenterTable::GetFocusCurves()
{
	return FocusCurves;
}

bool FImageCenterTable::DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance) const
{
	FImageCenterInfo ImageCenterInfo;
	if (GetPoint(InFocus, InZoom, ImageCenterInfo, InputTolerance))
	{
		return true;
	}

	return false;
}

const FBaseFocusPoint* FImageCenterTable::GetBaseFocusPoint(int32 InIndex) const
{
	if (FocusPoints.IsValidIndex(InIndex))
	{
		return &FocusPoints[InIndex];
	}

	return nullptr;
}

TMap<ELensDataCategory, FLinkPointMetadata> FImageCenterTable::GetLinkedCategories() const
{
	static TMap<ELensDataCategory, FLinkPointMetadata> LinkedToCategories =
	{
		{ELensDataCategory::Distortion, {true}},
		{ELensDataCategory::Zoom, {true}},
		{ELensDataCategory::STMap, {true}},
		{ELensDataCategory::NodalOffset, {false}},
	};
	return LinkedToCategories;
}

void FImageCenterTable::RemoveFocusPoint(float InFocus)
{
	LensDataTableUtils::RemoveFocusPoint(FocusPoints, InFocus);
	LensDataTableUtils::RemoveFocusFromFocusCurves(FocusCurves, InFocus);
}

bool FImageCenterTable::HasFocusPoint(float InFocus, float InputTolerance) const
{
	return DoesFocusPointExists(InFocus, InputTolerance);
}

void FImageCenterTable::ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	LensDataTableUtils::ChangeFocusPoint(FocusPoints, InExistingFocus, InNewFocus, InputTolerance);
	LensDataTableUtils::ChangeFocusInFocusCurves(FocusCurves, InExistingFocus, InNewFocus, InputTolerance);
}

void FImageCenterTable::MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance)
{
	LensDataTableUtils::MergeFocusPoint(FocusPoints, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
	LensDataTableUtils::MergeFocusInFocusCurves(FocusCurves, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
}

void FImageCenterTable::RemoveZoomPoint(float InFocus, float InZoom)
{
	LensDataTableUtils::RemoveZoomPoint(FocusPoints, InFocus, InZoom);
	LensDataTableUtils::RemoveZoomFromFocusCurves(FocusCurves, InFocus, InZoom);
}

bool FImageCenterTable::HasZoomPoint(float InFocus, float InZoom, float InputTolerance)
{
	return DoesZoomPointExists(InFocus, InZoom, InputTolerance);
}

void FImageCenterTable::ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance)
{
	LensDataTableUtils::ChangeZoomPoint(FocusPoints, InFocus, InExistingZoom, InNewZoom, InputTolerance);
	
	FImageCenterInfo Data;
	if (!GetPoint(InFocus, InNewZoom, Data, InputTolerance))
	{
		return;
	}

	LensDataTableUtils::ChangeZoomInFocusCurves(FocusCurves, InFocus, InExistingZoom, InNewZoom, Data, InputTolerance);
}

bool FImageCenterTable::AddPoint(float InFocus, float InZoom, const FImageCenterInfo& InData, float InputTolerance,
	bool bIsCalibrationPoint)
{
	if (!LensDataTableUtils::AddPoint(FocusPoints, InFocus, InZoom, InData, InputTolerance, bIsCalibrationPoint))
	{
		return false;
	}

	LensDataTableUtils::AddPointToFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

bool FImageCenterTable::GetPoint(const float InFocus, const float InZoom, FImageCenterInfo& OutData, float InputTolerance) const
{
	if (const FImageCenterFocusPoint* FocalLengthFocusPoint = GetFocusPoint(InFocus, InputTolerance))
	{
		FImageCenterInfo ImageCenterInfo;
		if (FocalLengthFocusPoint->GetPoint(InZoom, ImageCenterInfo, InputTolerance))
		{
			// Copy struct to outer
			OutData = ImageCenterInfo;
			return true;
		}
	}
	
	return false;
}

bool FImageCenterTable::SetPoint(float InFocus, float InZoom, const FImageCenterInfo& InData, float InputTolerance)
{
	if (!LensDataTableUtils::SetPoint(*this, InFocus, InZoom, InData, InputTolerance))
	{
		return false;
	}

	LensDataTableUtils::SetPointInFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	
	return true;
}

void FImageCenterTable::BuildFocusCurves()
{
	// Ensure that the focus curves are empty before building them from the table data
	FocusCurves.Empty();
	LensDataTableUtils::BuildFocusCurves(FocusPoints, FocusCurves);
}
