// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


/**
 * Set of templated functions operating on common APIed lens table data structure
 */
namespace LensDataTableUtils
{
	/** Removes a focus point from a container */
	template<typename FocusPointType>
	void RemoveFocusPoint(TArray<FocusPointType>& Container, float InFocus)
	{
		const int32 FoundIndex = Container.IndexOfByPredicate([InFocus](const FocusPointType& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus); });
    	if(FoundIndex != INDEX_NONE)
    	{
    		Container.RemoveAt(FoundIndex);
    	}
	}

	template<typename FocusCurveType>
	void RemoveFocusFromFocusCurves(TArray<FocusCurveType>& InFocusCurves, float InFocus)
	{
		for (FocusCurveType& Curve : InFocusCurves)
		{
			Curve.RemovePoint(InFocus, UE_SMALL_NUMBER);
		}

		InFocusCurves.RemoveAll([](FocusCurveType InCurve) { return InCurve.IsEmpty(); });
	}
	
	/** Changes the value of a focus point in the container */
	template<typename FocusPointType>
	void ChangeFocusPoint(TArray<FocusPointType>& Container, float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		const int32 FoundIndex = Container.IndexOfByPredicate([InExistingFocus, InputTolerance](const FocusPointType& Point)
		{
			return FMath::IsNearlyEqual(Point.Focus, InExistingFocus, InputTolerance);
		});
		
		if (FoundIndex != INDEX_NONE)
		{
			Container[FoundIndex].Focus = InNewFocus;
		}
	}

	template<typename FocusCurveType>
	void ChangeFocusInFocusCurves(TArray<FocusCurveType>& InFocusCurves, float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		for (FocusCurveType& Curve : InFocusCurves)
		{
			Curve.ChangeFocus(InExistingFocus, InNewFocus, InputTolerance);
		}
	}
	
	/** Merges the points in the specified source focus into the specified destination focus */
	template<typename FocusPointType>
	void MergeFocusPoint(TArray<FocusPointType>& Container, float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		const int32 SrcIndex = Container.IndexOfByPredicate([InSrcFocus, InputTolerance](const FocusPointType& Point)
		{
			return FMath::IsNearlyEqual(Point.Focus, InSrcFocus, InputTolerance);
		});
		
		const int32 DestIndex = Container.IndexOfByPredicate([InDestFocus, InputTolerance](const FocusPointType& Point)
		{
			return FMath::IsNearlyEqual(Point.Focus, InDestFocus, InputTolerance);
		});
		
		if (SrcIndex != INDEX_NONE)
		{
			if (DestIndex != INDEX_NONE)
			{
				FocusPointType& SrcPoint = Container[SrcIndex];
				FocusPointType& DestPoint = Container[DestIndex];

				for (int32 Index = 0; Index < SrcPoint.GetNumPoints(); ++Index)
				{
					float Zoom = SrcPoint.GetZoom(Index);

					typename FocusPointType::PointType SrcData;
					SrcPoint.GetPoint(Zoom, SrcData);
					bool bIsCalibrationPoint = SrcPoint.IsCalibrationPoint(Zoom);

					typename FocusPointType::PointType DestData;
					if (DestPoint.GetPoint(Zoom, DestData))
					{
						if (!bReplaceExistingZoomPoints)
						{
							continue;
						}

						DestPoint.RemovePoint(Zoom);
					}
					
					DestPoint.AddPoint(Zoom, SrcData, InputTolerance, bIsCalibrationPoint);
				}

				RemoveFocusPoint(Container, InSrcFocus);
			}
			else
			{
				// The destination doesn't exist, so we can just change the source focus point to the destination focus point
				ChangeFocusPoint(Container, InSrcFocus, InDestFocus);
			}
		}
	}
	
	template<typename FocusCurveType>
	void MergeFocusInFocusCurves(TArray<FocusCurveType>& InFocusCurves, float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		for (FocusCurveType& Curve : InFocusCurves)
		{
			Curve.MergeFocus(InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
		}
	}
	
	/** Gets all point info for specific data table */
	template<typename FPointInfoType, typename FDataTableType>
	TArray<FPointInfoType> GetAllPointsInfo(const FDataTableType& InTable)
	{
		TArray<FPointInfoType> PointsInfoType;

		for (const typename FDataTableType::FocusPointType& Point : InTable.FocusPoints)
		{
			// Get Focus Value
			const float FocusValue = Point.Focus;

			// Loop through all zoom points
			const int32 ZoomPointsNum = Point.GetNumPoints();
			for (int32 ZoomPointIndex = 0; ZoomPointIndex < ZoomPointsNum; ++ZoomPointIndex)
			{
				// Get Zoom Value
				const float ZoomValue = Point.GetZoom(ZoomPointIndex);

				// Zoom point should be valid
				typename FPointInfoType::TypeInfo PointInfo;
				ensure(InTable.GetPoint(FocusValue, ZoomValue, PointInfo));

				// Add Point into Array
				PointsInfoType.Add({FocusValue, ZoomValue, MoveTemp(PointInfo)});
			}
		}

		return PointsInfoType;
	}

	/** Removes a zoom point for a given focus value in a container */
	template<typename FocusPointType>
	void RemoveZoomPoint(TArray<FocusPointType>& Container, float InFocus, float InZoom)
	{
		bool bIsEmpty = false;
		const int32 FoundIndex = Container.IndexOfByPredicate([InFocus](const FocusPointType& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus); });
		if(FoundIndex != INDEX_NONE)
		{
			Container[FoundIndex].RemovePoint(InZoom);
			bIsEmpty = Container[FoundIndex].IsEmpty();
		}

		if(bIsEmpty)
		{
			Container.RemoveAt(FoundIndex);
		}
	}

	template<typename FocusCurveType>
	void RemoveZoomFromFocusCurves(TArray<FocusCurveType>& FocusCurves, float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		const int32 CurveIndex = FocusCurves.IndexOfByPredicate([InZoom, InputTolerance](const FocusCurveType& Curve)
		{
			return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance);
		});
		
		if (CurveIndex != INDEX_NONE)
		{
			FocusCurves[CurveIndex].RemovePoint(InFocus, InputTolerance);
			if (FocusCurves[CurveIndex].IsEmpty())
			{
				// If the curve has no points, that indicates there are no focuses that contain the zoom value, so we can delete it entirely
				FocusCurves.RemoveAt(CurveIndex);
			}
		}
	}
	
	/** Changes the value of a zoom point for a given focus value in a container */
	template<typename FocusPointType>
	void ChangeZoomPoint(TArray<FocusPointType>& Container, float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		const int32 FoundIndex = Container.IndexOfByPredicate([InFocus, InputTolerance](const FocusPointType& Point)
		{
			return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance);
		});
		
		if (FoundIndex != INDEX_NONE)
		{
			typename FocusPointType::PointType PointData;
			Container[FoundIndex].GetPoint(InExistingZoom, PointData);
			
			bool bIsCalibrationPoint = Container[FoundIndex].IsCalibrationPoint(InExistingZoom);
			
			Container[FoundIndex].RemovePoint(InExistingZoom);
			Container[FoundIndex].AddPoint(InNewZoom, PointData, InputTolerance, bIsCalibrationPoint);
		}
	}
	
	template<typename FocusCurveType, typename DataType>
	void ChangeZoomInFocusCurves(TArray<FocusCurveType>& FocusCurves, float InFocus, float InExistingZoom, float InNewZoom, const DataType& InData, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		RemoveZoomFromFocusCurves(FocusCurves, InFocus, InExistingZoom, InputTolerance);
		
		const int32 NewCurveIndex = FocusCurves.IndexOfByPredicate([InNewZoom, InputTolerance](const FocusCurveType& Curve)
		{
			return FMath::IsNearlyEqual(Curve.Zoom, InNewZoom, InputTolerance);
		});
		
		if (NewCurveIndex != INDEX_NONE)
		{
			FocusCurves[NewCurveIndex].AddPoint(InFocus, InData, InputTolerance);
			return;
		}

		// At this point, no focus curve has been found that matches the input zoom to the specified tolerance, so create one
		FocusCurveType NewCurve;
		NewCurve.Zoom = InNewZoom;
		NewCurve.AddPoint(InFocus, InData, InputTolerance);
		FocusCurves.Add(NewCurve);
	}
	
	/** Adds a point at a specified focus and zoom input values */
	template<typename FocusPointType, typename DataType>
	bool AddPoint(TArray<FocusPointType>& InContainer, float InFocus, float InZoom, const DataType& InData, float InputTolerance, bool bIsCalibrationPoint)
	{
		int32 PointIndex = 0;
		for (; PointIndex < InContainer.Num(); ++PointIndex)
		{
			FocusPointType& FocusPoint = InContainer[PointIndex];
			if (FMath::IsNearlyEqual(FocusPoint.Focus, InFocus, InputTolerance))
			{
				return FocusPoint.AddPoint(InZoom, InData, InputTolerance, bIsCalibrationPoint);
			}
			else if (InFocus < FocusPoint.Focus)
			{
				break;
			}
		}

		FocusPointType NewFocusPoint;
		NewFocusPoint.Focus = InFocus;
		const bool bSuccess = NewFocusPoint.AddPoint(InZoom, InData, InputTolerance, bIsCalibrationPoint);
		if(bSuccess)
		{
			InContainer.Insert(MoveTemp(NewFocusPoint), PointIndex);
		}

		return bSuccess;
	}

	/** Adds a point at the specified focus and zoom to a matching focus curve */
	template<typename FocusCurveType, typename DataType>
	void AddPointToFocusCurve(TArray<FocusCurveType>& InContainer, float InFocus, float InZoom, const DataType& InData, float InputTolerance)
	{
		int32 CurveIndex = 0;
		for (; CurveIndex < InContainer.Num(); ++CurveIndex)
		{
			FocusCurveType& FocusCurve = InContainer[CurveIndex];
			if (FMath::IsNearlyEqual(FocusCurve.Zoom, InZoom, InputTolerance))
			{
				FocusCurve.AddPoint(InFocus, InData, InputTolerance);
				return;
			}

			if (InZoom < FocusCurve.Zoom)
			{
				break;
			}
		}

		// At this point, no focus curve has been found that matches the input zoom to the specified tolerance, so create one
		FocusCurveType NewCurve;
		NewCurve.Zoom = InZoom;
		NewCurve.AddPoint(InFocus, InData, InputTolerance);
		InContainer.Insert(MoveTemp(NewCurve), CurveIndex);
	}
	
	template<typename TableType, typename DataType>
	bool SetPoint(TableType& InTable, float InFocus, float InZoom, const DataType& InData, float InputTolerance = KINDA_SMALL_NUMBER)
	{
		for (int32 PointIndex = 0; PointIndex < InTable.FocusPoints.Num() && InTable.FocusPoints[PointIndex].Focus <= InFocus; ++PointIndex)
		{
			typename TableType::FocusPointType& FocusPoint = InTable.FocusPoints[PointIndex];
			if (FMath::IsNearlyEqual(FocusPoint.Focus, InFocus, InputTolerance))
			{
				return FocusPoint.SetPoint(InZoom, InData);
			}
		}

		return false;
	}

	/** Sets a point at the specified focus and zoom to a matching focus curve */
	template<typename FocusCurveType, typename DataType>
	void SetPointInFocusCurve(TArray<FocusCurveType>& InContainer, float InFocus, float InZoom, const DataType& InData, float InputTolerance)
	{
		for (int32 CurveIndex = 0; CurveIndex < InContainer.Num(); ++CurveIndex)
		{
			FocusCurveType& FocusCurve = InContainer[CurveIndex];
			if (FMath::IsNearlyEqual(FocusCurve.Zoom, InZoom, InputTolerance))
			{
				FocusCurve.SetPoint(InFocus, InData, InputTolerance);
				return;
			}

			if (InZoom < FocusCurve.Zoom)
			{
				break;
			}
		}
	}
	
	/** Clears content of a table */
	template<typename Type>
	void EmptyTable(Type& InTable)
	{
		InTable.FocusPoints.Empty(0);
		InTable.FocusCurves.Empty(0);
	}

	struct FPointNeighbors
	{
		/** Returns true if the previous index is identical to the next index */
		bool IsSinglePoint() const { return PreviousIndex == NextIndex; }
		
		int32 PreviousIndex = INDEX_NONE;
		int32 NextIndex = INDEX_NONE;
	};

	/** Finds indices of neighbor focus points for a given focus value */
	template<typename Type>
	FPointNeighbors FindFocusPoints(float InFocus, TConstArrayView<Type> Container)
	{
		FPointNeighbors Neighbors;
		if (Container.Num() <= 0)
		{
			return Neighbors;
		}

		for (int32 Index = 0; Index < Container.Num(); ++Index)
		{
			const Type& Point = Container[Index];
			if (Point.Focus > InFocus)
			{
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = FMath::Max(Index - 1, 0);
				break;
			}
			else if (FMath::IsNearlyEqual(Point.Focus, InFocus))
			{
				//We found a point exactly matching the desired one
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = Index;
				break;
			}
		}

		//We haven't found a point, default to last one
		if (Neighbors.PreviousIndex == INDEX_NONE && Neighbors.NextIndex == INDEX_NONE)
		{
			Neighbors.NextIndex = Container.Num() - 1;
			Neighbors.PreviousIndex = Container.Num() - 1;
		}

		return Neighbors;
	}

	template<typename Type>
	FPointNeighbors FindFocusCurves(float InZoom, TConstArrayView<Type> Container)
	{
		FPointNeighbors Neighbors;
		if (Container.Num() <= 0)
		{
			return Neighbors;
		}

		for (int32 Index = 0; Index < Container.Num(); ++Index)
		{
			const Type& Curve = Container[Index];
			if (Curve.Zoom > InZoom)
			{
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = FMath::Max(Index - 1, 0);
				break;
			}
			
			if (FMath::IsNearlyEqual(Curve.Zoom, InZoom))
			{
				//We found a point exactly matching the desired one
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = Index;
				break;
			}
		}

		//We haven't found a curve, default to last one
		if (Neighbors.PreviousIndex == INDEX_NONE && Neighbors.NextIndex == INDEX_NONE)
		{
			Neighbors.NextIndex = Container.Num() - 1;
			Neighbors.PreviousIndex = Container.Num() - 1;
		}

		return Neighbors;
	}
	
	/** Finds indices of neighbor zoom points for a given zoom value */
	template<typename Type>
	FPointNeighbors FindZoomPoints(float InZoom, const TArray<Type>& Container)
	{
		FPointNeighbors Neighbors;
		if (Container.Num() <= 0)
		{
			return Neighbors;
		}

		for (int32 Index = 0; Index < Container.Num(); ++Index)
		{
			const Type& Point = Container[Index];
			if (Point.Zoom > InZoom)
			{
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = FMath::Max(Index - 1, 0);
				break;
			}
			else if (FMath::IsNearlyEqual(Point.Zoom, InZoom))
			{
				//We found a point exactly matching the desired one
				Neighbors.NextIndex = Index;
				Neighbors.PreviousIndex = Index;
				break;
			}
		}

		//We haven't found a point, default to last one
		if (Neighbors.PreviousIndex == INDEX_NONE && Neighbors.NextIndex == INDEX_NONE)
		{
			Neighbors.NextIndex = Container.Num() - 1;
			Neighbors.PreviousIndex = Container.Num() - 1;
		}

		return Neighbors;
	}

	/** Finds a points that matches input Focus and Zoom and returns its value.
	 * Returns false if point isn't found
	 */
	template<typename FocusPointType, typename DataType>
	bool GetPointValue(float InFocus, float InZoom, TConstArrayView<FocusPointType> Container, DataType& OutData)
	{
		const FPointNeighbors FocusNeighbors = FindFocusPoints(InFocus, Container);

		if (FocusNeighbors.PreviousIndex != FocusNeighbors.NextIndex)
		{
			return false;
		}

		if (FocusNeighbors.PreviousIndex == INDEX_NONE)
		{
			return false;
		}

		const FPointNeighbors ZoomNeighbors = FindZoomPoints(InZoom, Container[FocusNeighbors.PreviousIndex].ZoomPoints);

		if (ZoomNeighbors.PreviousIndex != ZoomNeighbors.NextIndex)
		{
			return false;
		}

		if (ZoomNeighbors.PreviousIndex == INDEX_NONE)
		{
			return false;
		}

		return Container[FocusNeighbors.PreviousIndex].GetValue(ZoomNeighbors.PreviousIndex, OutData);
	}

	/** Get total number of Zoom points for all Focus points of this data table */
	template<typename FocusPointType>
	int32 GetTotalPointNum(const TArray<FocusPointType>& Container)
	{
		int32 PointNum = 0;
	
		for (const FocusPointType& Point : Container)
		{
			PointNum += Point.GetNumPoints();
		}

		return PointNum;
	}

	template<typename FocusPointType, typename FocusCurveType>
	void BuildFocusCurves(const TArray<FocusPointType>& InPoints, TArray<FocusCurveType>& OutCurves)
	{
		for (const FocusPointType& FocusPoint : InPoints)
		{
			for (int32 Index = 0; Index < FocusPoint.GetNumPoints(); ++Index)
			{
				const float Zoom = FocusPoint.GetZoom(Index);

				typename FocusPointType::PointType PointData;
				if (FocusPoint.GetPoint(Zoom, PointData))
				{
					int32 ExistingCurveIndex = 0;
					for (; ExistingCurveIndex < OutCurves.Num(); ++ExistingCurveIndex)
					{
						FocusCurveType& Curve = OutCurves[ExistingCurveIndex];
						if (FMath::IsNearlyEqual(Curve.Zoom, Zoom))
						{
							Curve.AddPoint(FocusPoint.Focus, PointData);
							break;
						}

						if (Zoom < Curve.Zoom)
						{
							FocusCurveType NewCurve;
							NewCurve.Zoom = Zoom;
							NewCurve.AddPoint(FocusPoint.Focus, PointData);
							OutCurves.Insert(NewCurve, ExistingCurveIndex);
							break;
						}
					}

					if (ExistingCurveIndex == OutCurves.Num())
					{
						FocusCurveType NewCurve;
						NewCurve.Zoom = Zoom;
						NewCurve.AddPoint(FocusPoint.Focus, PointData);
						OutCurves.Add(NewCurve);
					}
				}
			}
		}
	}
}
