// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tables/NodalOffsetTable.h"

#include "LensFile.h"
#include "LensTableUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NodalOffsetTable)

int32 FNodalOffsetFocusPoint::GetNumPoints() const
{
	return LocationOffset[0].GetNumKeys();
}

float FNodalOffsetFocusPoint::GetZoom(int32 Index) const
{
	return LocationOffset[0].Keys[Index].Time;
}

bool FNodalOffsetFocusPoint::GetPoint(float InZoom, FNodalPointOffset& OutData, float InputTolerance) const
{	
	for(int32 Index = 0; Index < LocationDimension; ++Index)
	{
		FKeyHandle Handle = LocationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			OutData.LocationOffset[Index] = LocationOffset[Index].GetKeyValue(Handle);
		}
		else
		{
			return false;
		}
	}

	FRotator Rotator;
	for(int32 Index = 0; Index < RotationDimension; ++Index)
	{
		FKeyHandle Handle = RotationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			Rotator.SetComponentForAxis(static_cast<EAxis::Type>(Index+1), RotationOffset[Index].GetKeyValue(Handle));
		}
		else
		{
			return false;
		}
	}

	OutData.RotationOffset = Rotator.Quaternion();

	return true;
}

bool FNodalOffsetFocusPoint::AddPoint(float InZoom, const FNodalPointOffset& InData, float InputTolerance, bool /** bIsCalibrationPoint */)
{
	for(int32 Index = 0; Index < LocationDimension; ++Index)
	{
		FKeyHandle Handle = LocationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			LocationOffset[Index].SetKeyValue(Handle, InData.LocationOffset[Index]);	
		}
		else
		{
			Handle = LocationOffset[Index].AddKey(InZoom, InData.LocationOffset[Index]);
			LocationOffset[Index].SetKeyTangentMode(Handle, ERichCurveTangentMode::RCTM_Auto);
			LocationOffset[Index].SetKeyInterpMode(Handle, RCIM_Cubic);
		}
	}

	const FRotator NewRotator = InData.RotationOffset.Rotator();
	for(int32 Index = 0; Index < RotationDimension; ++Index)
	{
		FKeyHandle Handle = RotationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			RotationOffset[Index].SetKeyValue(Handle, NewRotator.GetComponentForAxis(static_cast<EAxis::Type>(Index+1)));	
		}
		else
		{
			Handle = RotationOffset[Index].AddKey(InZoom, NewRotator.GetComponentForAxis(static_cast<EAxis::Type>(Index+1)));
			RotationOffset[Index].SetKeyTangentMode(Handle, ERichCurveTangentMode::RCTM_Auto);
			RotationOffset[Index].SetKeyInterpMode(Handle, RCIM_Cubic);
		}
	}
	return true;
}

bool FNodalOffsetFocusPoint::SetPoint(float InZoom, const FNodalPointOffset& InData, float InputTolerance)
{
	for(int32 Index = 0; Index < LocationDimension; ++Index)
	{
		FKeyHandle Handle = LocationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			LocationOffset[Index].SetKeyValue(Handle, InData.LocationOffset[Index]);	
		}
		else
		{
			return false;
		}
	}

	const FRotator NewRotator = InData.RotationOffset.Rotator();
	for(int32 Index = 0; Index < RotationDimension; ++Index)
	{
		FKeyHandle Handle = RotationOffset[Index].FindKey(InZoom, InputTolerance);
		if(Handle != FKeyHandle::Invalid())
		{
			RotationOffset[Index].SetKeyValue(Handle, NewRotator.GetComponentForAxis(static_cast<EAxis::Type>(Index+1)));	
		}
		else
		{
			return false;
		}
	}
	return true;
}

void FNodalOffsetFocusPoint::RemovePoint(float InZoomValue)
{
	for(int32 Index = 0; Index < LocationDimension; ++Index)
	{
		const FKeyHandle KeyHandle = LocationOffset[Index].FindKey(InZoomValue);
		if(KeyHandle != FKeyHandle::Invalid())
		{
			LocationOffset[Index].DeleteKey(KeyHandle);
		}
	}

	for(int32 Index = 0; Index < RotationDimension; ++Index)
	{
		const FKeyHandle KeyHandle = RotationOffset[Index].FindKey(InZoomValue);
		if(KeyHandle != FKeyHandle::Invalid())
		{
			RotationOffset[Index].DeleteKey(KeyHandle);
		}
	}
}

bool FNodalOffsetFocusPoint::IsEmpty() const
{
	return LocationOffset[0].IsEmpty();
}

const FRichCurve* FNodalOffsetFocusPoint::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FNodalOffsetTable::FParameters::IsValidComposed(InParameterIndex))
	{
		int32 Parameter;
		EAxis::Type Axis;
		FNodalOffsetTable::FParameters::Decompose(InParameterIndex, Parameter, Axis);

		if (Parameter == FNodalOffsetTable::FParameters::Location)
		{
			return &LocationOffset[Axis - 1];
		}

		if (Parameter == FNodalOffsetTable::FParameters::Rotation)
		{
			return &RotationOffset[Axis - 1];
		}
	}

	return nullptr;
}

FRichCurve* FNodalOffsetFocusPoint::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FNodalOffsetFocusPoint*>(this)->GetCurveForParameter(InParameterIndex));
}

void FNodalOffsetFocusCurve::AddPoint(float InFocus, const FNodalPointOffset& InData, float InputTolerance)
{
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::LocationDimension; ++Index)
	{
		AddPointToCurve(LocationOffset[Index], InFocus, InData.LocationOffset[Index], InputTolerance);
	}

	const FRotator NewRotator = InData.RotationOffset.Rotator();
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::RotationDimension; ++Index)
	{
		AddPointToCurve(RotationOffset[Index], InFocus, NewRotator.GetComponentForAxis(static_cast<EAxis::Type>(Index+1)), InputTolerance);
	}
}

void FNodalOffsetFocusCurve::SetPoint(float InFocus, const FNodalPointOffset& InData, float InputTolerance)
{
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::LocationDimension; ++Index)
	{
		SetPointInCurve(LocationOffset[Index], InFocus, InData.LocationOffset[Index], InputTolerance);
	}

	const FRotator NewRotator = InData.RotationOffset.Rotator();
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::RotationDimension; ++Index)
	{
		SetPointInCurve(RotationOffset[Index], InFocus, NewRotator.GetComponentForAxis(static_cast<EAxis::Type>(Index+1)), InputTolerance);
	}
}

void FNodalOffsetFocusCurve::RemovePoint(float InFocus, float InputTolerance)
{
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::LocationDimension; ++Index)
	{
		DeletePointFromCurve(LocationOffset[Index], InFocus, InputTolerance);
	}
	
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::RotationDimension; ++Index)
	{
		DeletePointFromCurve(RotationOffset[Index], InFocus, InputTolerance);
	}
}

void FNodalOffsetFocusCurve::ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::LocationDimension; ++Index)
	{
		ChangeFocusInCurve(LocationOffset[Index], InExistingFocus, InNewFocus, InputTolerance);
	}
	
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::RotationDimension; ++Index)
	{
		ChangeFocusInCurve(RotationOffset[Index], InExistingFocus, InNewFocus, InputTolerance);
	}
}

void FNodalOffsetFocusCurve::MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::LocationDimension; ++Index)
	{
		MergeFocusInCurve(LocationOffset[Index], InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
	}
	
	for (int32 Index = 0; Index < FNodalOffsetFocusPoint::RotationDimension; ++Index)
	{
		MergeFocusInCurve(RotationOffset[Index], InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
	}
}

bool FNodalOffsetFocusCurve::IsEmpty() const
{
	return !LocationOffset[0].GetNumKeys();
}

const FRichCurve* FNodalOffsetFocusCurve::GetCurveForParameter(int32 InParameterIndex) const
{
	if (FNodalOffsetTable::FParameters::IsValidComposed(InParameterIndex))
	{
		int32 Parameter;
		EAxis::Type Axis;
		FNodalOffsetTable::FParameters::Decompose(InParameterIndex, Parameter, Axis);

		if (Parameter == FNodalOffsetTable::FParameters::Location)
		{
			return &LocationOffset[Axis - 1];
		}

		if (Parameter == FNodalOffsetTable::FParameters::Rotation)
		{
			return &RotationOffset[Axis - 1];
		}
	}

	return nullptr;
}

FRichCurve* FNodalOffsetFocusCurve::GetCurveForParameter(int32 InParameterIndex)
{
	return const_cast<FRichCurve*>(const_cast<const FNodalOffsetFocusCurve*>(this)->GetCurveForParameter(InParameterIndex));
}

bool FNodalOffsetTable::DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance) const
{
	FNodalPointOffset NodalPointOffset;
	if (GetPoint(InFocus, InZoom, NodalPointOffset, InputTolerance))
	{
		return true;
	}

	return false;
}

const FBaseFocusPoint* FNodalOffsetTable::GetBaseFocusPoint(int32 InIndex) const
{
	if (FocusPoints.IsValidIndex(InIndex))
	{
		return &FocusPoints[InIndex];
	}

	return nullptr;
}

TMap<ELensDataCategory, FLinkPointMetadata> FNodalOffsetTable::GetLinkedCategories() const
{
	static TMap<ELensDataCategory, FLinkPointMetadata> LinkedToCategories =
	{
		{ELensDataCategory::Distortion, {false}},
		{ELensDataCategory::Zoom, {false}},
		{ELensDataCategory::STMap, {false}},
		{ELensDataCategory::ImageCenter, {false}},
	};
	return LinkedToCategories;
}

int32 FNodalOffsetTable::GetTotalPointNum() const
{
	return LensDataTableUtils::GetTotalPointNum(FocusPoints);
}

UScriptStruct* FNodalOffsetTable::GetScriptStruct() const
{
	return StaticStruct();
}

bool FNodalOffsetTable::BuildParameterCurveAtFocus(float InFocus, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return false;
	}

	int32 Parameter;
	EAxis::Type Axis;
	FParameters::Decompose(InParameterIndex, Parameter, Axis);
	
	if (const FNodalOffsetFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		if (Parameter == FParameters::Location)
		{
			OutCurve = FocusPoint->LocationOffset[Axis - 1];
		}
		else
		{
			OutCurve = FocusPoint->RotationOffset[Axis - 1];
		}
		
		return true;
	}

	return false;
}

bool FNodalOffsetTable::BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return false;
	}

	int32 Parameter;
	EAxis::Type Axis;
	FParameters::Decompose(InParameterIndex, Parameter, Axis);

	if (const FNodalOffsetFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		if (Parameter == FParameters::Location)
		{
			OutCurve = FocusCurve->LocationOffset[Axis - 1];
		}
		else
		{
			OutCurve = FocusCurve->RotationOffset[Axis - 1];
		}
		
		return true;
	}

	return false;
}

void FNodalOffsetTable::SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return;
	}
	
	if (FNodalOffsetFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		CopyCurveKeys(InSourceCurve, *FocusPoint->GetCurveForParameter(InParameterIndex), InKeys);
		PropagateCurveValuesToCrossCurves(*FocusPoint->GetCurveForParameter(InParameterIndex), InFocus, [this, InParameterIndex](float InZoom)->FRichCurve*
		{
			if (FNodalOffsetFocusCurve* Curve = GetFocusCurve(InZoom))
			{
				return Curve->GetCurveForParameter(InParameterIndex);
			}

			return nullptr;
		});
	}
}

void FNodalOffsetTable::SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return;
	}

	if (FNodalOffsetFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		CopyCurveKeys(InSourceCurve, *FocusCurve->GetCurveForParameter(InParameterIndex), InKeys);
		PropagateCurveValuesToCrossCurves(*FocusCurve->GetCurveForParameter(InParameterIndex), InZoom, [this, InParameterIndex](float InFocus)->FRichCurve*
		{
			if (FNodalOffsetFocusPoint* Point = GetFocusPoint(InFocus))
			{
				return Point->GetCurveForParameter(InParameterIndex);
			}

			return nullptr;
		});
	}
}

FText FNodalOffsetTable::GetParameterValueLabel(int32 InParameterIndex) const
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return FText();
	}
	
	int32 Parameter;
	EAxis::Type Axis;
	FParameters::Decompose(InParameterIndex, Parameter, Axis);
	
	if (Parameter == FParameters::Location)
	{
		return NSLOCTEXT("FNodalOffsetTable", "LocationParameterValueLabel", "(cm)");
	}
	else
	{
		return NSLOCTEXT("FNodalOffsetTable", "RotationParameterValueLabel", "(deg)");
	}
}

FText FNodalOffsetTable::GetParameterValueUnitLabel(int32 InParameterIndex) const
{
	if (!FParameters::IsValidComposed(InParameterIndex))
	{
		return FText();
	}
	
	int32 Parameter;
	EAxis::Type Axis;
	FParameters::Decompose(InParameterIndex, Parameter, Axis);
	
	if (Parameter == FParameters::Location)
	{
		return NSLOCTEXT("FNodalOffsetTable", "LocationParameterUnitLabel", "cm");
	}
	else
	{
		return NSLOCTEXT("FNodalOffsetTable", "RotationParameterUnitLabel", "deg");
	}
}

const FNodalOffsetFocusPoint* FNodalOffsetTable::GetFocusPoint(float InFocus, float InputTolerance) const
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FNodalOffsetFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

FNodalOffsetFocusPoint* FNodalOffsetTable::GetFocusPoint(float InFocus, float InputTolerance)
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FNodalOffsetFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

const FNodalOffsetFocusCurve* FNodalOffsetTable::GetFocusCurve(float InZoom, float InputTolerance) const
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FNodalOffsetFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

FNodalOffsetFocusCurve* FNodalOffsetTable::GetFocusCurve(float InZoom, float InputTolerance)
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FNodalOffsetFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

TConstArrayView<FNodalOffsetFocusPoint> FNodalOffsetTable::GetFocusPoints() const
{
	return FocusPoints;
}

TArray<FNodalOffsetFocusPoint>& FNodalOffsetTable::GetFocusPoints()
{
	return FocusPoints;
}

TConstArrayView<FNodalOffsetFocusCurve> FNodalOffsetTable::GetFocusCurves() const
{
	return FocusCurves;
}

TArray<FNodalOffsetFocusCurve>& FNodalOffsetTable::GetFocusCurves()
{
	return FocusCurves;
}

void FNodalOffsetTable::ForEachPoint(FFocusPointCallback InCallback) const
{
	for (const FNodalOffsetFocusPoint& Point : FocusPoints)
	{
		InCallback(Point);
	}
}

void FNodalOffsetTable::RemoveFocusPoint(float InFocus)
{
	LensDataTableUtils::RemoveFocusPoint(FocusPoints, InFocus);
	LensDataTableUtils::RemoveFocusFromFocusCurves(FocusCurves, InFocus);
}

bool FNodalOffsetTable::HasFocusPoint(float InFocus, float InputTolerance) const
{
	return DoesFocusPointExists(InFocus, InputTolerance);
}

void FNodalOffsetTable::ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	LensDataTableUtils::ChangeFocusPoint(FocusPoints, InExistingFocus, InNewFocus, InputTolerance);
	LensDataTableUtils::ChangeFocusInFocusCurves(FocusCurves, InExistingFocus, InNewFocus, InputTolerance);
}

void FNodalOffsetTable::MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance)
{
	LensDataTableUtils::MergeFocusPoint(FocusPoints, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
	LensDataTableUtils::MergeFocusInFocusCurves(FocusCurves, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
}

void FNodalOffsetTable::RemoveZoomPoint(float InFocus, float InZoom)
{
	LensDataTableUtils::RemoveZoomPoint(FocusPoints, InFocus, InZoom);
	LensDataTableUtils::RemoveZoomFromFocusCurves(FocusCurves, InFocus, InZoom);
}

bool FNodalOffsetTable::HasZoomPoint(float InFocus, float InZoom, float InputTolerance)
{
	return DoesZoomPointExists(InFocus, InZoom, InputTolerance);
}

void FNodalOffsetTable::ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance)
{
	LensDataTableUtils::ChangeZoomPoint(FocusPoints, InFocus, InExistingZoom, InNewZoom, InputTolerance);
	
	FNodalPointOffset Data;
	if (!GetPoint(InFocus, InNewZoom, Data, InputTolerance))
	{
		return;
	}

	LensDataTableUtils::ChangeZoomInFocusCurves(FocusCurves, InFocus, InExistingZoom, InNewZoom, Data, InputTolerance);
}

bool FNodalOffsetTable::DoesFocusPointExists(float InFocus, float InputTolerance) const
{
	if (GetFocusPoint(InFocus, InputTolerance) != nullptr)
	{
		return true;
	}

	return false;
}

bool FNodalOffsetTable::AddPoint(float InFocus, float InZoom, const FNodalPointOffset& InData, float InputTolerance, bool bIsCalibrationPoint)
{
	if (!LensDataTableUtils::AddPoint(FocusPoints, InFocus, InZoom, InData, InputTolerance, bIsCalibrationPoint))
	{
		return false;
	}

	LensDataTableUtils::AddPointToFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

bool FNodalOffsetTable::GetPoint(const float InFocus, const float InZoom, FNodalPointOffset& OutData, float InputTolerance) const
{
	if (const FNodalOffsetFocusPoint* NodalOffsetFocusPoint = GetFocusPoint(InFocus, InputTolerance))
	{
		FNodalPointOffset NodalPointOffset;
		if (NodalOffsetFocusPoint->GetPoint(InZoom, NodalPointOffset, InputTolerance))
		{
			// Copy struct to outer
			OutData = NodalPointOffset;
			return true;
		}
	}
	
	return false;
}

bool FNodalOffsetTable::SetPoint(float InFocus, float InZoom, const FNodalPointOffset& InData, float InputTolerance)
{
	if (!LensDataTableUtils::SetPoint(*this, InFocus, InZoom, InData, InputTolerance))
	{
		return false;
	}

	LensDataTableUtils::SetPointInFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	
	return true;
}

void FNodalOffsetTable::BuildFocusCurves()
{
	// Ensure that the focus curves are empty before building them from the table data
	FocusCurves.Empty();
	LensDataTableUtils::BuildFocusCurves(FocusPoints, FocusCurves);
}

