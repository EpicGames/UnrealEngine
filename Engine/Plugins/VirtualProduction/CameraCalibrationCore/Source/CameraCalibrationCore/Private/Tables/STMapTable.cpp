// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tables/STMapTable.h"

#include "LensFile.h"
#include "LensTableUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(STMapTable)

int32 FSTMapFocusPoint::GetNumPoints() const
{
	return MapBlendingCurve.GetNumKeys();
}

float FSTMapFocusPoint::GetZoom(int32 Index) const
{
	return MapBlendingCurve.Keys[Index].Time;
}

const FSTMapZoomPoint* FSTMapFocusPoint::GetZoomPoint(float InZoom) const
{
	return ZoomPoints.FindByPredicate([InZoom](const FSTMapZoomPoint& Point) { return FMath::IsNearlyEqual(Point.Zoom, InZoom); });
}

FSTMapZoomPoint* FSTMapFocusPoint::GetZoomPoint(float InZoom)
{
	return ZoomPoints.FindByPredicate([InZoom](const FSTMapZoomPoint& Point) { return FMath::IsNearlyEqual(Point.Zoom, InZoom); });
}

bool FSTMapFocusPoint::GetPoint(float InZoom, FSTMapInfo& OutData, float /*InputTolerance*/) const
{
	if (const FSTMapZoomPoint* STMapZoomPoint = GetZoomPoint(InZoom))
	{
		OutData = STMapZoomPoint->STMapInfo;
		return true;
	}

	return false;
}

bool FSTMapFocusPoint::AddPoint(float InZoom, const FSTMapInfo& InData, float InputTolerance, bool bIsCalibrationPoint)
{
	const FKeyHandle Handle = MapBlendingCurve.FindKey(InZoom, InputTolerance);
	if(Handle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = MapBlendingCurve.GetIndexSafe(Handle);
		if(ensure(ZoomPoints.IsValidIndex(PointIndex)))
		{
			//No need to update map curve since x == y
			ZoomPoints[PointIndex].STMapInfo = InData;
			ZoomPoints[PointIndex].DerivedDistortionData.bIsDirty = true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		//Add new zoom point
		const FKeyHandle NewKeyHandle = MapBlendingCurve.AddKey(InZoom, InZoom);
		MapBlendingCurve.SetKeyTangentMode(NewKeyHandle, ERichCurveTangentMode::RCTM_Auto);
		MapBlendingCurve.SetKeyInterpMode(NewKeyHandle, RCIM_Cubic);

		const int32 KeyIndex = MapBlendingCurve.GetIndexSafe(NewKeyHandle);
		FSTMapZoomPoint NewZoomPoint;
		NewZoomPoint.Zoom = InZoom;
		NewZoomPoint.STMapInfo = InData;
		NewZoomPoint.bIsCalibrationPoint = bIsCalibrationPoint;
		ZoomPoints.Insert(MoveTemp(NewZoomPoint), KeyIndex);
	}

	return true;
}

bool FSTMapFocusPoint::SetPoint(float InZoom, const FSTMapInfo& InData, float InputTolerance)
{
	const FKeyHandle Handle = MapBlendingCurve.FindKey(InZoom, InputTolerance);
	if(Handle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = MapBlendingCurve.GetIndexSafe(Handle);
		if(ensure(ZoomPoints.IsValidIndex(PointIndex)))
		{
			//No need to update map curve since x == y
			ZoomPoints[PointIndex].STMapInfo = InData;
			ZoomPoints[PointIndex].DerivedDistortionData.bIsDirty = true;
			return true;
		}
	}

	return false;
}

bool FSTMapFocusPoint::IsCalibrationPoint(float InZoom, float InputTolerance)
{
	const FKeyHandle Handle = MapBlendingCurve.FindKey(InZoom, InputTolerance);
	if(Handle != FKeyHandle::Invalid())
	{
		const int32 PointIndex = MapBlendingCurve.GetIndexSafe(Handle);
		if(ensure(ZoomPoints.IsValidIndex(PointIndex)))
		{
			return ZoomPoints[PointIndex].bIsCalibrationPoint;
		}
	}

	return false;
}

void FSTMapFocusPoint::RemovePoint(float InZoomValue)
{
	const int32 FoundIndex = ZoomPoints.IndexOfByPredicate([InZoomValue](const FSTMapZoomPoint& Point) { return FMath::IsNearlyEqual(Point.Zoom, InZoomValue); });
	if(FoundIndex != INDEX_NONE)
	{
		ZoomPoints.RemoveAt(FoundIndex);
	}

	const FKeyHandle KeyHandle = MapBlendingCurve.FindKey(InZoomValue);
	if(KeyHandle != FKeyHandle::Invalid())
	{
		MapBlendingCurve.DeleteKey(KeyHandle);
	}
}

bool FSTMapFocusPoint::IsEmpty() const
{
	return MapBlendingCurve.IsEmpty();
}

void FSTMapFocusCurve::AddPoint(float InFocus, const FSTMapInfo& InData, float InputTolerance)
{
	AddPointToCurve(MapBlendingCurve, InFocus, InFocus, InputTolerance);
}

void FSTMapFocusCurve::SetPoint(float InFocus, const FSTMapInfo& InData, float InputTolerance)
{
	// No need to update map curve since x == y
}

void FSTMapFocusCurve::RemovePoint(float InFocus, float InputTolerance)
{
	DeletePointFromCurve(MapBlendingCurve, InFocus, InputTolerance);
}

void FSTMapFocusCurve::ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	ChangeFocusInCurve(MapBlendingCurve, InExistingFocus, InNewFocus, InputTolerance);
}

void FSTMapFocusCurve::MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance)
{
	MergeFocusInCurve(MapBlendingCurve, InExistingFocus, InNewFocus, bReplaceExisting, InputTolerance);
}

bool FSTMapFocusCurve::IsEmpty() const
{
	return !MapBlendingCurve.GetNumKeys();
}

void FSTMapTable::ForEachPoint(FFocusPointCallback InCallback) const
{
	for (const FSTMapFocusPoint& Point : FocusPoints)
	{
		InCallback(Point);
	}
}

bool FSTMapTable::DoesFocusPointExists(float InFocus, float InputTolerance) const
{
	if (GetFocusPoint(InFocus, InputTolerance) != nullptr)
	{
		return true;
	}

	return false;
}

bool FSTMapTable::DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance) const
{
	FSTMapInfo STMapInfo;
	if (GetPoint(InFocus, InZoom, STMapInfo, InputTolerance))
	{
		return true;
	}

	return false;
}

const FBaseFocusPoint* FSTMapTable::GetBaseFocusPoint(int32 InIndex) const
{
	if (FocusPoints.IsValidIndex(InIndex))
	{
		return &FocusPoints[InIndex];
	}

	return nullptr;
}

int32 FSTMapTable::GetTotalPointNum() const
{
	return LensDataTableUtils::GetTotalPointNum(FocusPoints);
}

UScriptStruct* FSTMapTable::GetScriptStruct() const
{
	return StaticStruct();
}

bool FSTMapTable::BuildParameterCurveAtFocus(float InFocus, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (const FSTMapFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		OutCurve = FocusPoint->MapBlendingCurve;
		return true;
	}

	return false;
}

bool FSTMapTable::BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const
{
	if (const FSTMapFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		OutCurve = FocusCurve->MapBlendingCurve;
		return true;
	}

	return false;
}

void FSTMapTable::SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (FSTMapFocusPoint* FocusPoint = GetFocusPoint(InFocus))
	{
		CopyCurveKeys(InSourceCurve, FocusPoint->MapBlendingCurve, InKeys);
	}
}

void FSTMapTable::SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys)
{
	if (FSTMapFocusCurve* FocusCurve = GetFocusCurve(InZoom))
	{
		CopyCurveKeys(InSourceCurve, FocusCurve->MapBlendingCurve, InKeys);
	}
}

const FSTMapFocusPoint* FSTMapTable::GetFocusPoint(float InFocus, float InputTolerance) const
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FSTMapFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

FSTMapFocusPoint* FSTMapTable::GetFocusPoint(float InFocus, float InputTolerance)
{
	return FocusPoints.FindByPredicate([InFocus, InputTolerance](const FSTMapFocusPoint& Point) { return FMath::IsNearlyEqual(Point.Focus, InFocus, InputTolerance); });
}

const FSTMapFocusCurve* FSTMapTable::GetFocusCurve(float InZoom, float InputTolerance) const
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FSTMapFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

FSTMapFocusCurve* FSTMapTable::GetFocusCurve(float InZoom, float InputTolerance)
{
	return FocusCurves.FindByPredicate([InZoom, InputTolerance](const FSTMapFocusCurve& Curve) { return FMath::IsNearlyEqual(Curve.Zoom, InZoom, InputTolerance); });
}

TConstArrayView<FSTMapFocusPoint> FSTMapTable::GetFocusPoints() const
{
	return FocusPoints;
}

TArrayView<FSTMapFocusPoint> FSTMapTable::GetFocusPoints()
{
	return FocusPoints;
}

TConstArrayView<FSTMapFocusCurve> FSTMapTable::GetFocusCurves() const
{
	return FocusCurves;
}

TArray<FSTMapFocusCurve>& FSTMapTable::GetFocusCurves()
{
	return FocusCurves;
}

void FSTMapTable::RemoveFocusPoint(float InFocus)
{
	LensDataTableUtils::RemoveFocusPoint(FocusPoints, InFocus);
	LensDataTableUtils::RemoveFocusFromFocusCurves(FocusCurves, InFocus);
}

bool FSTMapTable::HasFocusPoint(float InFocus, float InputTolerance) const
{
	return DoesFocusPointExists(InFocus, InputTolerance);
}

void FSTMapTable::ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance)
{
	LensDataTableUtils::ChangeFocusPoint(FocusPoints, InExistingFocus, InNewFocus, InputTolerance);
	LensDataTableUtils::ChangeFocusInFocusCurves(FocusCurves, InExistingFocus, InNewFocus, InputTolerance);
}

void FSTMapTable::MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance)
{
	LensDataTableUtils::MergeFocusPoint(FocusPoints, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
	LensDataTableUtils::MergeFocusInFocusCurves(FocusCurves, InSrcFocus, InDestFocus, bReplaceExistingZoomPoints, InputTolerance);
}

void FSTMapTable::RemoveZoomPoint(float InFocus, float InZoom)
{
	LensDataTableUtils::RemoveZoomPoint(FocusPoints, InFocus, InZoom);
	LensDataTableUtils::RemoveZoomFromFocusCurves(FocusCurves, InFocus, InZoom);
}

bool FSTMapTable::HasZoomPoint(float InFocus, float InZoom, float InputTolerance)
{
	return DoesZoomPointExists(InFocus, InZoom, InputTolerance);
}

void FSTMapTable::ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance)
{
	LensDataTableUtils::ChangeZoomPoint(FocusPoints, InFocus, InExistingZoom, InNewZoom, InputTolerance);
	
	FSTMapInfo Data;
	if (!GetPoint(InFocus, InNewZoom, Data, InputTolerance))
	{
		return;
	}

	LensDataTableUtils::ChangeZoomInFocusCurves(FocusCurves, InFocus, InExistingZoom, InNewZoom, Data, InputTolerance);
}

TMap<ELensDataCategory, FLinkPointMetadata> FSTMapTable::GetLinkedCategories() const
{
	static TMap<ELensDataCategory, FLinkPointMetadata> LinkedToCategories =
	{
		{ELensDataCategory::Zoom, {true}},
		{ELensDataCategory::ImageCenter, {true}},
		{ELensDataCategory::NodalOffset, {false}},
	};
	return LinkedToCategories;
}

bool FSTMapTable::AddPoint(float InFocus, float InZoom, const FSTMapInfo& InData, float InputTolerance,
                           bool bIsCalibrationPoint)
{
	if (!LensDataTableUtils::AddPoint(FocusPoints, InFocus, InZoom, InData, InputTolerance, bIsCalibrationPoint))
	{
		return false;
	}

	LensDataTableUtils::AddPointToFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	return true;
}

bool FSTMapTable::GetPoint(const float InFocus, const float InZoom, FSTMapInfo& OutData, float InputTolerance) const
{
	if (const FSTMapFocusPoint* STMapFocusPoint = GetFocusPoint(InFocus, InputTolerance))
	{
		FSTMapInfo SMapInfo;
		if (STMapFocusPoint->GetPoint(InZoom, SMapInfo, InputTolerance))
		{
			// Copy struct to outer
			OutData = SMapInfo;
			return true;
		}
	}
	
	return false;
}

bool FSTMapTable::SetPoint(float InFocus, float InZoom, const FSTMapInfo& InData, float InputTolerance)
{
	if (!LensDataTableUtils::SetPoint(*this, InFocus, InZoom, InData, InputTolerance))
	{
		return false;
	}

	LensDataTableUtils::SetPointInFocusCurve(FocusCurves, InFocus, InZoom, InData, InputTolerance);
	
	return true;
}

void FSTMapTable::BuildFocusCurves()
{
	// Ensure that the focus curves are empty before building them from the table data
	FocusCurves.Empty();
	LensDataTableUtils::BuildFocusCurves(FocusPoints, FocusCurves);
}

