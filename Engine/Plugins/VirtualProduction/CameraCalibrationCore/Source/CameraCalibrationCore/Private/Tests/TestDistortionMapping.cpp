// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"

#include "CameraCalibrationSubsystem.h"
#include "CameraCalibrationTypes.h"
#include "Dom/JsonObject.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "LensFile.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SphericalLensDistortionModelHandler.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

// Set this cvar to re-generate the reference test file when the test is run.
// Clear it to run the test normally (which will compare test outputs to the values in the reference file).
static TAutoConsoleVariable<bool> CVarCameraCalibrationGenerateTestReferences(
	TEXT("CameraCalibrationCore.GenerateTestReferences"),
	false,
	TEXT("When true, camera calibration tests will generate reference data files instead of verifying against them."),
	ECVF_Default
);

namespace CameraCalibrationTestUtil
{

	/** Data structure for the tests */
	struct FLensData
	{
		FLensData() = default;
		FLensData(float InFocus, float InZoom, FDistortionInfo InDistortionInfo, FImageCenterInfo InImageCenter, FFocalLengthInfo InFocalLength)
			: Focus(InFocus)
			, Zoom(InZoom)
			, Distortion(InDistortionInfo)
			, ImageCenter(InImageCenter)
			, FocalLength(InFocalLength)
		{}
		
		float Focus;
		float Zoom;
		FDistortionInfo Distortion;
		FImageCenterInfo ImageCenter;
		FFocalLengthInfo FocalLength;
	};

	/** Structure representing a single test case with all its data */
	struct FTestCase
	{
		/** Helps know what this test case is about */
		FString Name;

		/** Focus value to evaluate at */
		float Focus;

		/** Zoom value to evaluate at */
		float Zoom;
		
		
		/** Expected result that we'll be comparing with */
		TArray<float> ExpectedResult;
		
		/** Additional metadata needed to conduct the test. */
		FString DataType;         // e.g. "Distortion", "ImageCenter", "FocalLength", etc.
		FString EvaluationMethod; // "LensFile", "Handler", etc.
	};

	/** Collection of test cases including calibration points */
	struct FTestSuite
	{
		/** Helps understand what this is about. */
		FString SuiteName;

		/** The complete input calibration points */
		TArray<FLensData> CalibrationData;

		/** The actual test cases being conducted */
		TArray<FTestCase> TestCases;
	};

	/** Gets the first valid world (looks for Editor world if GIsEditor). Required to create UObjects during the test. */
	UWorld* GetFirstWorld()
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
#if WITH_EDITOR
			if (GIsEditor)
			{
				if(Context.WorldType == EWorldType::Editor)
				{
					return Context.World();
				}
			}
			else
			{
				if(Context.World() != nullptr)
				{
					return Context.World();
				}
			}
#else
			if(Context.World() != nullptr)
			{
				return Context.World();
			}
#endif
		}

		return nullptr;
	}

	/** Safely extract a number from JSON with validation and error reporting */
	bool SafeGetNumberFromJson(const TSharedPtr<FJsonObject>& JsonObj, const FString& FieldName, float& OutValue, float DefaultValue = 0.0f)
	{
		if (!JsonObj.IsValid())
		{
			UE_LOGF(LogTemp, Warning, "Invalid JSON object when getting field '%ls'", *FieldName);
			OutValue = DefaultValue;
			return false;
		}
		
		double TempValue;
		if (JsonObj->TryGetNumberField(FieldName, TempValue))
		{
			OutValue = static_cast<float>(TempValue);
			return true;
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "Field '%ls' is not a valid number in JSON, using default value %f", *FieldName, DefaultValue);
			OutValue = DefaultValue;
			return false;
		}
	}

	/** Safely extract a number from JSON value with validation and error reporting */
	bool SafeGetNumberFromJsonValue(const TSharedPtr<FJsonValue>& JsonValue, float& OutValue, float DefaultValue = 0.0f)
	{
		if (!JsonValue.IsValid())
		{
			UE_LOGF(LogTemp, Warning, "Invalid JSON value when parsing number");
			OutValue = DefaultValue;
			return false;
		}
		
		double TempValue;
		if (JsonValue->TryGetNumber(TempValue))
		{
			OutValue = static_cast<float>(TempValue);
			return true;
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "JSON value is not a valid number, using default value %f", DefaultValue);
			OutValue = DefaultValue;
			return false;
		}
	}

	/** Utility to get the path for test reference files within the plugin directory */
	FString GetTestReferenceFilePath(const FString& TestName)
	{
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("CameraCalibrationCore"));
		if (!Plugin.IsValid())
		{
			return FString();
		}
		
		FString PluginDir = Plugin->GetBaseDir();
		return PluginDir / TEXT("Tests") / TEXT("References") / (TestName + TEXT(".json"));
	}

	/** Save test results to a JSON reference file */
	template<typename T>
	bool SaveTestReference(const FString& TestName, const FString& SectionName, TConstArrayView<T> Values)
	{
		FString FilePath = GetTestReferenceFilePath(TestName);
		if (FilePath.IsEmpty())
		{
			return false;
		}
		
		// Ensure directory exists
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(FilePath), true);
		
		// Load existing JSON or create new one

		TSharedPtr<FJsonObject> RootObject;
		FString ExistingContent;

		if (FFileHelper::LoadFileToString(ExistingContent, *FilePath))
		{
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ExistingContent);
			FJsonSerializer::Deserialize(Reader, RootObject);
		}
		
		if (!RootObject.IsValid())
		{
			RootObject = MakeShareable(new FJsonObject);
		}
		
		// Add the values array to the section

		TArray<TSharedPtr<FJsonValue>> JsonValues;

		for (const T& Value : Values)
		{
			if constexpr (std::is_same_v<T, float> || std::is_same_v<T, double>)
			{
				JsonValues.Add(MakeShareable(new FJsonValueNumber(Value)));
			}
			else if constexpr (std::is_same_v<T, FVector2D::FReal>)
			{
				JsonValues.Add(MakeShareable(new FJsonValueNumber(static_cast<double>(Value))));
			}
		}
		
		RootObject->SetArrayField(SectionName, JsonValues);
		
		// Write JSON back to file

		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);

		FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);

		if (FFileHelper::SaveStringToFile(OutputString, *FilePath))
		{
			UE_LOGF(LogTemp, Log, "Saved test reference data to: %ls", *FilePath);
			return true;
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Failed to save test reference data to: %ls", *FilePath);
			return false;
		}
	}

	/** Load test reference values from JSON file */
	template<typename T>
	bool LoadTestReference(const FString& TestName, const FString& SectionName, TArray<T>& OutValues)
	{
		FString FilePath = GetTestReferenceFilePath(TestName);

		if (FilePath.IsEmpty())
		{
			return false;
		}
		
		FString FileContent;

		if (!FFileHelper::LoadFileToString(FileContent, *FilePath))
		{
			return false;
		}
		
		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);

		if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
		{
			return false;
		}
		
		const TArray<TSharedPtr<FJsonValue>>* JsonValues;

		if (!RootObject->TryGetArrayField(SectionName, JsonValues))
		{
			return false;
		}
		
		OutValues.Empty();

		for (const TSharedPtr<FJsonValue>& JsonValue : *JsonValues)
		{
			if constexpr (std::is_same_v<T, float>)
			{
				double DoubleValue;

				if (JsonValue->TryGetNumber(DoubleValue))
				{
					OutValues.Add(static_cast<float>(DoubleValue));
				}
			}
			else if constexpr (std::is_same_v<T, double>)
			{
				double DoubleValue;

				if (JsonValue->TryGetNumber(DoubleValue))
				{
					OutValues.Add(DoubleValue);
				}
			}
			else if constexpr (std::is_same_v<T, FVector2D::FReal>)
			{
				double DoubleValue;

				if (JsonValue->TryGetNumber(DoubleValue))
				{
					OutValues.Add(static_cast<FVector2D::FReal>(DoubleValue));
				}
			}
		}
		
		return OutValues.Num() > 0;
	}


	/** Convert FLensData to JSON object */
	TSharedPtr<FJsonObject> LensDataToJson(const FLensData& LensData)
	{
		TSharedPtr<FJsonObject> LensDataObj = MakeShareable(new FJsonObject);
		LensDataObj->SetNumberField(TEXT("focus"), LensData.Focus);
		LensDataObj->SetNumberField(TEXT("zoom"), LensData.Zoom);
		
		// Distortion data
		TSharedPtr<FJsonObject> DistortionObj = MakeShareable(new FJsonObject);
		TArray<TSharedPtr<FJsonValue>> DistortionParams;
		for (float Param : LensData.Distortion.Parameters)
		{
			DistortionParams.Add(MakeShareable(new FJsonValueNumber(Param)));
		}
		DistortionObj->SetArrayField(TEXT("parameters"), DistortionParams);
		LensDataObj->SetObjectField(TEXT("distortion"), DistortionObj);
		
		// Image center data
		TSharedPtr<FJsonObject> ImageCenterObj = MakeShareable(new FJsonObject);
		TSharedPtr<FJsonObject> PrincipalPointObj = MakeShareable(new FJsonObject);
		PrincipalPointObj->SetNumberField(TEXT("x"), LensData.ImageCenter.PrincipalPoint.X);
		PrincipalPointObj->SetNumberField(TEXT("y"), LensData.ImageCenter.PrincipalPoint.Y);
		ImageCenterObj->SetObjectField(TEXT("principalPoint"), PrincipalPointObj);
		LensDataObj->SetObjectField(TEXT("imageCenter"), ImageCenterObj);
		
		// Focal length data
		TSharedPtr<FJsonObject> FocalLengthObj = MakeShareable(new FJsonObject);
		TSharedPtr<FJsonObject> FxFyObj = MakeShareable(new FJsonObject);
		FxFyObj->SetNumberField(TEXT("x"), LensData.FocalLength.FxFy.X);
		FxFyObj->SetNumberField(TEXT("y"), LensData.FocalLength.FxFy.Y);
		FocalLengthObj->SetObjectField(TEXT("fxFy"), FxFyObj);
		LensDataObj->SetObjectField(TEXT("focalLength"), FocalLengthObj);
		
		return LensDataObj;
	}

	/** Convert JSON object to FLensData */
	FLensData JsonToLensData(const TSharedPtr<FJsonObject>& JsonObj)
	{
		FLensData LensData;
		SafeGetNumberFromJson(JsonObj, TEXT("focus"), LensData.Focus);
		SafeGetNumberFromJson(JsonObj, TEXT("zoom"), LensData.Zoom);
		
		// Distortion data
		const TSharedPtr<FJsonObject>* DistortionObj;
		if (JsonObj->TryGetObjectField(TEXT("distortion"), DistortionObj))
		{
			const TArray<TSharedPtr<FJsonValue>>* DistortionParams;
			if ((*DistortionObj)->TryGetArrayField(TEXT("parameters"), DistortionParams))
			{
				for (const TSharedPtr<FJsonValue>& Param : *DistortionParams)
				{
					float ParamValue;
					if (SafeGetNumberFromJsonValue(Param, ParamValue))
					{
						LensData.Distortion.Parameters.Add(ParamValue);
					}
					else
					{
						// Add 0.0f if parsing failed (already logged warning in SafeGetNumberFromJsonValue)
						LensData.Distortion.Parameters.Add(0.0f);
					}
				}
			}
		}
		
		// Image center data
		const TSharedPtr<FJsonObject>* ImageCenterObj;
		if (JsonObj->TryGetObjectField(TEXT("imageCenter"), ImageCenterObj))
		{
			const TSharedPtr<FJsonObject>* PrincipalPointObj;
			if ((*ImageCenterObj)->TryGetObjectField(TEXT("principalPoint"), PrincipalPointObj))
			{
				float X, Y;
				SafeGetNumberFromJson(*PrincipalPointObj, TEXT("x"), X);
				SafeGetNumberFromJson(*PrincipalPointObj, TEXT("y"), Y);
				LensData.ImageCenter.PrincipalPoint.X = X;
				LensData.ImageCenter.PrincipalPoint.Y = Y;
			}
		}
		
		// Focal length data
		const TSharedPtr<FJsonObject>* FocalLengthObj;
		if (JsonObj->TryGetObjectField(TEXT("focalLength"), FocalLengthObj))
		{
			const TSharedPtr<FJsonObject>* FxFyObj;
			if ((*FocalLengthObj)->TryGetObjectField(TEXT("fxFy"), FxFyObj))
			{
				float X, Y;
				SafeGetNumberFromJson(*FxFyObj, TEXT("x"), X);
				SafeGetNumberFromJson(*FxFyObj, TEXT("y"), Y);
				LensData.FocalLength.FxFy.X = X;
				LensData.FocalLength.FxFy.Y = Y;
			}
		}
		
		return LensData;
	}

	/** Save test suite to JSON file */
	bool SaveTestSuite(const FString& TestName, const FTestSuite& TestSuite)
	{
		FString FilePath = GetTestReferenceFilePath(TestName);
		if (FilePath.IsEmpty())
		{
			return false;
		}
		
		// Ensure directory exists
		IFileManager::Get().MakeDirectory(*FPaths::GetPath(FilePath), true);
		
		// Create JSON structure
		TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		RootObject->SetStringField(TEXT("suiteName"), TestSuite.SuiteName);
		
		// Add calibration data array
		TArray<TSharedPtr<FJsonValue>> CalibrationDataArray;
		for (const FLensData& LensData : TestSuite.CalibrationData)
		{
			CalibrationDataArray.Add(MakeShareable(new FJsonValueObject(LensDataToJson(LensData))));
		}
		RootObject->SetArrayField(TEXT("calibrationData"), CalibrationDataArray);
		
		TArray<TSharedPtr<FJsonValue>> TestCaseArray;
		for (const FTestCase& TestCase : TestSuite.TestCases)
		{
			TSharedPtr<FJsonObject> TestCaseObj = MakeShareable(new FJsonObject);
			TestCaseObj->SetStringField(TEXT("name"), TestCase.Name);
			TestCaseObj->SetNumberField(TEXT("focus"), TestCase.Focus);
			TestCaseObj->SetNumberField(TEXT("zoom"), TestCase.Zoom);
			TestCaseObj->SetStringField(TEXT("dataType"), TestCase.DataType);
			TestCaseObj->SetStringField(TEXT("evaluationMethod"), TestCase.EvaluationMethod);
			
			// Helper lambda to convert float array to JSON array
			auto FloatArrayToJson = [](const TArray<float>& FloatArray) -> TArray<TSharedPtr<FJsonValue>>
			{
				TArray<TSharedPtr<FJsonValue>> JsonArray;
				for (float Value : FloatArray)
				{
					JsonArray.Add(MakeShareable(new FJsonValueNumber(Value)));
				}
				return JsonArray;
			};
			
			
			// Add expected result
			TestCaseObj->SetArrayField(TEXT("expectedResult"), FloatArrayToJson(TestCase.ExpectedResult));
			
			TestCaseArray.Add(MakeShareable(new FJsonValueObject(TestCaseObj)));
		}
		
		RootObject->SetArrayField(TEXT("testCases"), TestCaseArray);
		
		// Write JSON to file

		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer);

		if (FFileHelper::SaveStringToFile(OutputString, *FilePath))
		{
			UE_LOGF(LogTemp, Log, "Saved test suite to: %ls", *FilePath);
			return true;
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Failed to save test suite to: %ls", *FilePath);
			return false;
		}
	}

	/** Load test suite from JSON file */
	bool LoadTestSuite(const FString& TestName, FTestSuite& OutTestSuite)
	{
		FString FilePath = GetTestReferenceFilePath(TestName);
		if (FilePath.IsEmpty())
		{
			return false;
		}
		
		FString FileContent;
		if (!FFileHelper::LoadFileToString(FileContent, *FilePath))
		{
			return false;
		}
		
		TSharedPtr<FJsonObject> RootObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContent);
		if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
		{
			return false;
		}
		
		// Parse suite name
		OutTestSuite.SuiteName = RootObject->GetStringField(TEXT("suiteName"));
		
		// Parse calibration data
		const TArray<TSharedPtr<FJsonValue>>* CalibrationDataArray;
		if (RootObject->TryGetArrayField(TEXT("calibrationData"), CalibrationDataArray))
		{
			OutTestSuite.CalibrationData.Empty();
			for (const TSharedPtr<FJsonValue>& CalibrationValue : *CalibrationDataArray)
			{
				const TSharedPtr<FJsonObject>* CalibrationObj;
				if (CalibrationValue->TryGetObject(CalibrationObj))
				{
					OutTestSuite.CalibrationData.Add(JsonToLensData(*CalibrationObj));
				}
			}
		}
		
		// Parse test cases
		const TArray<TSharedPtr<FJsonValue>>* TestCaseArray;
		if (!RootObject->TryGetArrayField(TEXT("testCases"), TestCaseArray))
		{
			return false;
		}
		
		// Helper lambda to convert JSON array to float array
		auto JsonArrayToFloatArray = [](const TArray<TSharedPtr<FJsonValue>>& JsonArray, TArray<float>& OutArray)
		{
			OutArray.Empty();
			for (const TSharedPtr<FJsonValue>& JsonValue : JsonArray)
			{
				double DoubleValue;
				if (JsonValue->TryGetNumber(DoubleValue))
				{
					OutArray.Add(static_cast<float>(DoubleValue));
				}
			}
		};
		
		OutTestSuite.TestCases.Empty();

		for (const TSharedPtr<FJsonValue>& JsonValue : *TestCaseArray)
		{
			const TSharedPtr<FJsonObject>* TestCaseObj;
			if (!JsonValue->TryGetObject(TestCaseObj))
			{
				continue;
			}
			
			FTestCase TestCase;
			TestCase.Name = (*TestCaseObj)->GetStringField(TEXT("name"));
			SafeGetNumberFromJson(*TestCaseObj, TEXT("focus"), TestCase.Focus);
			SafeGetNumberFromJson(*TestCaseObj, TEXT("zoom"), TestCase.Zoom);
			TestCase.DataType = (*TestCaseObj)->GetStringField(TEXT("dataType"));
			TestCase.EvaluationMethod = (*TestCaseObj)->GetStringField(TEXT("evaluationMethod"));
			
			
			// Load expected result
			const TArray<TSharedPtr<FJsonValue>>* SourceArray;
			if ((*TestCaseObj)->TryGetArrayField(TEXT("expectedResult"), SourceArray))
			{
				JsonArrayToFloatArray(*SourceArray, TestCase.ExpectedResult);
			}
			
			OutTestSuite.TestCases.Add(TestCase);
		}
		
		return OutTestSuite.TestCases.Num() > 0;
	}

	/** Execute a single test case and verify results */
	void ExecuteTestCase(FAutomationTestBase& Test, const FTestCase& TestCase, const TArray<float>& ActualResult)
	{
		Test.TestEqual(FString::Printf(TEXT("%s array size"), *TestCase.Name), ActualResult.Num(), TestCase.ExpectedResult.Num());
		
		int32 NumToCheck = FMath::Min(ActualResult.Num(), TestCase.ExpectedResult.Num());
		for (int32 Index = 0; Index < NumToCheck; ++Index)
		{
			Test.TestEqual(FString::Printf(TEXT("%s[%d]"), *TestCase.Name, Index), ActualResult[Index], TestCase.ExpectedResult[Index]);
		}
	}

	/** Build a LensFile from calibration data for testing */
	ULensFile* BuildLensFileFromCalibrationData(const TArray<FLensData>& CalibrationData)
	{
		ULensFile* LensFile = NewObject<ULensFile>();
		if (!LensFile)
		{
			return nullptr;
		}
		
		// Set default lens info
		LensFile->LensInfo.SensorDimensions = FVector2D(36.0f, 20.25f); // Standard full-frame sensor
		
		// Add calibration points
		for (const FLensData& Data : CalibrationData)
		{
			// Add distortion data
			LensFile->DistortionTable.AddPoint(Data.Focus, Data.Zoom, Data.Distortion, 0.0f, true);
			
			// Add image center data  
			LensFile->ImageCenterTable.AddPoint(Data.Focus, Data.Zoom, Data.ImageCenter, 0.0f, true);
			
			// Add focal length data
			LensFile->FocalLengthTable.AddPoint(Data.Focus, Data.Zoom, Data.FocalLength, 0.0f, true);
		}
		
		// Build focus curves for interpolation
		LensFile->DistortionTable.BuildFocusCurves();
		LensFile->ImageCenterTable.BuildFocusCurves();
		LensFile->FocalLengthTable.BuildFocusCurves();
		
		return LensFile;
	}

	/** Run all test cases from a loaded test suite */

	void RunTestSuite(FAutomationTestBase& Test, const FString& TestName, ULensFile* ExternalLensFile, USphericalLensDistortionModelHandler* Handler)
	{
		FTestSuite TestSuite;
		if (!LoadTestSuite(TestName, TestSuite))
		{
			Test.AddError(FString::Printf(TEXT("Failed to load test suite: %s. Run with CameraCalibrationCore.GenerateTestReferences=1 to generate test definitions."), *TestName));
			return;
		}
		
		// Build LensFile from stored calibration data for complete test isolation
		ULensFile* LensFile = nullptr;
		if (TestSuite.CalibrationData.Num() > 0)
		{
			LensFile = BuildLensFileFromCalibrationData(TestSuite.CalibrationData);
			if (!LensFile)
			{
				Test.AddError(FString::Printf(TEXT("Failed to build LensFile from calibration data for test suite: %s"), *TestName));
				return;
			}
			Test.AddInfo(FString::Printf(TEXT("Built LensFile from %d calibration points for complete test isolation"), TestSuite.CalibrationData.Num()));
		}
		else
		{
			// Fall back to external LensFile if no calibration data (for backward compatibility)
			LensFile = ExternalLensFile;
			if (!LensFile)
			{
				Test.AddError(FString::Printf(TEXT("No calibration data found in test suite and no external LensFile provided: %s"), *TestName));
				return;
			}
		}
		
		UE_LOGF(LogTemp, Log, "Running test suite '%ls' with %d test cases", *TestSuite.SuiteName, TestSuite.TestCases.Num());
		
		for (const FTestCase& TestCase : TestSuite.TestCases)
		{
			TArray<float> ActualResult;
			
			// Evaluate based on test case parameters
			if (TestCase.DataType == TEXT("Distortion"))
			{
				FDistortionInfo DistortionInfo;
				LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(DistortionInfo.Parameters);
				LensFile->EvaluateDistortionParameters(TestCase.Focus, TestCase.Zoom, DistortionInfo);
				ActualResult = DistortionInfo.Parameters;
				
				if (TestCase.EvaluationMethod == TEXT("Handler") && Handler)
				{
					LensFile->EvaluateDistortionData(TestCase.Focus, TestCase.Zoom, LensFile->LensInfo.SensorDimensions, Handler);
					ActualResult = Handler->GetCurrentDistortionState().DistortionInfo.Parameters;
				}
			}
			else if (TestCase.DataType == TEXT("ImageCenter"))
			{
				FImageCenterInfo ImageCenter;
				LensFile->EvaluateImageCenterParameters(TestCase.Focus, TestCase.Zoom, ImageCenter);
				ActualResult = {static_cast<float>(ImageCenter.PrincipalPoint.X), static_cast<float>(ImageCenter.PrincipalPoint.Y)};
				
				if (TestCase.EvaluationMethod == TEXT("Handler") && Handler)
				{
					LensFile->EvaluateDistortionData(TestCase.Focus, TestCase.Zoom, LensFile->LensInfo.SensorDimensions, Handler);
					const FVector2D& HandlerCenter = Handler->GetCurrentDistortionState().ImageCenter.PrincipalPoint;
					ActualResult = {static_cast<float>(HandlerCenter.X), static_cast<float>(HandlerCenter.Y)};
				}
			}
			else if (TestCase.DataType == TEXT("FocalLength"))
			{
				FFocalLengthInfo FocalLength;
				LensFile->EvaluateFocalLength(TestCase.Focus, TestCase.Zoom, FocalLength);
				ActualResult = {static_cast<float>(FocalLength.FxFy.X), static_cast<float>(FocalLength.FxFy.Y)};
				
				if (TestCase.EvaluationMethod == TEXT("Handler") && Handler)
				{
					LensFile->EvaluateDistortionData(TestCase.Focus, TestCase.Zoom, LensFile->LensInfo.SensorDimensions, Handler);
					const FVector2D& HandlerFocal = Handler->GetCurrentDistortionState().FocalLengthInfo.FxFy;
					ActualResult = {static_cast<float>(HandlerFocal.X), static_cast<float>(HandlerFocal.Y)};
				}
			}
			
			ExecuteTestCase(Test, TestCase, ActualResult);
		}
	}

	/** Helper to create a test case from evaluation results */
	FTestCase CreateTestCase(const FString& Name, float Focus, float Zoom, const FString& DataType, const FString& EvaluationMethod, const TArray<float>& ActualResult)
	{
		FTestCase TestCase;
		TestCase.Name = Name;
		TestCase.Focus = Focus;
		TestCase.Zoom = Zoom;
		TestCase.DataType = DataType;
		TestCase.EvaluationMethod = EvaluationMethod;
		TestCase.ExpectedResult = ActualResult;
		return TestCase;
	}

	/** Generate comprehensive test suite from current lens data */
	void GenerateTestSuite(FAutomationTestBase& Test, const FString& TestName, ULensFile* LensFile, USphericalLensDistortionModelHandler* Handler, const TArray<FLensData>& InputData)
	{
		FTestSuite TestSuite;
		TestSuite.SuiteName = TestName;
		TestSuite.CalibrationData = InputData;  // Store the complete calibration data
		
		// Test parameters
		const FLensData& Input0 = InputData[0];
		const FLensData& Input1 = InputData[1];
		const FLensData& Input2 = InputData[2];
		const FLensData& Input3 = InputData[3];
		
		// Generate exact corner tests
		for (int32 InputIndex = 0; InputIndex < InputData.Num(); ++InputIndex)
		{
			const FLensData& Input = InputData[InputIndex];
			
			// Evaluate current data
			FDistortionInfo EvaluatedDistortion;
			LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(EvaluatedDistortion.Parameters);
			FImageCenterInfo EvaluatedImageCenter;
			FFocalLengthInfo EvaluatedFocalLength;
			
			LensFile->EvaluateDistortionParameters(Input.Focus, Input.Zoom, EvaluatedDistortion);
			LensFile->EvaluateImageCenterParameters(Input.Focus, Input.Zoom, EvaluatedImageCenter);
			LensFile->EvaluateFocalLength(Input.Focus, Input.Zoom, EvaluatedFocalLength);
			LensFile->EvaluateDistortionData(Input.Focus, Input.Zoom, LensFile->LensInfo.SensorDimensions, Handler);
			
			// Create corner test cases
			TestSuite.TestCases.Add(CreateTestCase(
				FString::Printf(TEXT("ExactCorner_Distortion_Focus%d_Zoom%d"), (int32)(Input.Focus * 10), (int32)(Input.Zoom * 10)),
				Input.Focus, Input.Zoom, TEXT("Distortion"), TEXT("LensFile"),
				EvaluatedDistortion.Parameters
			));
			
			TestSuite.TestCases.Add(CreateTestCase(
				FString::Printf(TEXT("ExactCorner_ImageCenter_Focus%d_Zoom%d"), (int32)(Input.Focus * 10), (int32)(Input.Zoom * 10)),
				Input.Focus, Input.Zoom, TEXT("ImageCenter"), TEXT("LensFile"),
				{static_cast<float>(EvaluatedImageCenter.PrincipalPoint.X), static_cast<float>(EvaluatedImageCenter.PrincipalPoint.Y)}
			));
			
			TestSuite.TestCases.Add(CreateTestCase(
				FString::Printf(TEXT("ExactCorner_FocalLength_Focus%d_Zoom%d"), (int32)(Input.Focus * 10), (int32)(Input.Zoom * 10)),
				Input.Focus, Input.Zoom, TEXT("FocalLength"), TEXT("LensFile"),
				{static_cast<float>(EvaluatedFocalLength.FxFy.X), static_cast<float>(EvaluatedFocalLength.FxFy.Y)}
			));
			
			// Handler variants
			TestSuite.TestCases.Add(CreateTestCase(
				FString::Printf(TEXT("ExactCorner_HandlerDistortion_Focus%d_Zoom%d"), (int32)(Input.Focus * 10), (int32)(Input.Zoom * 10)),
				Input.Focus, Input.Zoom, TEXT("Distortion"), TEXT("Handler"),
				Handler->GetCurrentDistortionState().DistortionInfo.Parameters
			));
		}
		
		// Generate single curve interpolation tests
		FDistortionInfo BlendedDistortion;
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		FImageCenterInfo BlendedImageCenter;
		FFocalLengthInfo BlendedFocalLength;
		
		// Focus 0, Zoom 0.25 (zoom curve interpolation)
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(Input0.Focus, 0.25f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(Input0.Focus, 0.25f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(Input0.Focus, 0.25f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_Distortion_Focus0_Zoom025"),
			Input0.Focus, 0.25f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_ImageCenter_Focus0_Zoom025"),
			Input0.Focus, 0.25f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		// Generate more single curve interpolation tests along zoom curves
		
		// Focus 0, Zoom 0.75 (zoom curve interpolation)
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(Input0.Focus, 0.75f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(Input0.Focus, 0.75f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(Input0.Focus, 0.75f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_Distortion_Focus0_Zoom075"),
			Input0.Focus, 0.75f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_ImageCenter_Focus0_Zoom075"),
			Input0.Focus, 0.75f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_FocalLength_Focus0_Zoom075"),
			Input0.Focus, 0.75f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Focus 1, Zoom 0.25 (zoom curve interpolation at other focus)
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(Input2.Focus, 0.25f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(Input2.Focus, 0.25f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(Input2.Focus, 0.25f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_Distortion_Focus1_Zoom025"),
			Input2.Focus, 0.25f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_ImageCenter_Focus1_Zoom025"),
			Input2.Focus, 0.25f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_FocalLength_Focus1_Zoom025"),
			Input2.Focus, 0.25f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Generate single curve interpolation tests along focus curves
		
		// Focus 0.25, Zoom 0 (focus curve interpolation)
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.25f, Input0.Zoom, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.25f, Input0.Zoom, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.25f, Input0.Zoom, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_Distortion_Focus025_Zoom0"),
			0.25f, Input0.Zoom, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_ImageCenter_Focus025_Zoom0"),
			0.25f, Input0.Zoom, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_FocalLength_Focus025_Zoom0"),
			0.25f, Input0.Zoom, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Focus 0.75, Zoom 1 (focus curve interpolation at other zoom)
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.75f, Input1.Zoom, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.75f, Input1.Zoom, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.75f, Input1.Zoom, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_Distortion_Focus075_Zoom1"),
			0.75f, Input1.Zoom, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_ImageCenter_Focus075_Zoom1"),
			0.75f, Input1.Zoom, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("SingleCurve_FocalLength_Focus075_Zoom1"),
			0.75f, Input1.Zoom, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Generate dual curve tests
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.25f, 0.25f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.25f, 0.25f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.25f, 0.25f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_Distortion_Focus025_Zoom025"),
			0.25f, 0.25f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_ImageCenter_Focus025_Zoom025"),
			0.25f, 0.25f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_FocalLength_Focus025_Zoom025"),
			0.25f, 0.25f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Additional dual curve tests at different interpolation points
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.75f, 0.25f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.75f, 0.25f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.75f, 0.25f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_Distortion_Focus075_Zoom025"),
			0.75f, 0.25f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_ImageCenter_Focus075_Zoom025"),
			0.75f, 0.25f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_FocalLength_Focus075_Zoom025"),
			0.75f, 0.25f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Test interpolation in the other direction
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.25f, 0.75f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.25f, 0.75f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.25f, 0.75f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_Distortion_Focus025_Zoom075"),
			0.25f, 0.75f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_ImageCenter_Focus025_Zoom075"),
			0.25f, 0.75f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_FocalLength_Focus025_Zoom075"),
			0.25f, 0.75f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Test center interpolation
		LensFile->LensInfo.LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(BlendedDistortion.Parameters);
		LensFile->EvaluateDistortionParameters(0.5f, 0.5f, BlendedDistortion);
		LensFile->EvaluateImageCenterParameters(0.5f, 0.5f, BlendedImageCenter);
		LensFile->EvaluateFocalLength(0.5f, 0.5f, BlendedFocalLength);
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_Distortion_Focus05_Zoom05"),
			0.5f, 0.5f, TEXT("Distortion"), TEXT("LensFile"),
			BlendedDistortion.Parameters
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_ImageCenter_Focus05_Zoom05"),
			0.5f, 0.5f, TEXT("ImageCenter"), TEXT("LensFile"),
			{static_cast<float>(BlendedImageCenter.PrincipalPoint.X), static_cast<float>(BlendedImageCenter.PrincipalPoint.Y)}
		));
		
		TestSuite.TestCases.Add(CreateTestCase(
			TEXT("DualCurve_FocalLength_Focus05_Zoom05"),
			0.5f, 0.5f, TEXT("FocalLength"), TEXT("LensFile"),
			{static_cast<float>(BlendedFocalLength.FxFy.X), static_cast<float>(BlendedFocalLength.FxFy.Y)}
		));
		
		// Save the complete test suite
		if (SaveTestSuite(TestName, TestSuite))
		{
			Test.AddInfo(FString::Printf(TEXT("Generated %d test cases for suite '%s'"), TestSuite.TestCases.Num(), *TestSuite.SuiteName));
		}
		else
		{
			Test.AddError(FString::Printf(TEXT("Failed to save test suite '%s' to disk"), *TestSuite.SuiteName));
		}
	}

	template<typename T>
	void TestEvaluationResult(FAutomationTestBase& Test, TConstArrayView<T> InExpected, TConstArrayView<T> InResult)
	{
		for(int32 Index = 0; Index < InResult.Num(); ++Index)
		{
			Test.TestEqual(FString::Printf(TEXT("Parameter[%d] should be equal to %0.2f"), Index, InExpected[Index]), InResult[Index], InExpected[Index]);
		}	
	}

	/** Enhanced dual curve test that uses reference files when CVar is enabled */
	template<typename T>
	void TestDualCurveEvaluationResultWithReference(FAutomationTestBase& Test, const FString& TestName, const FString& SectionName, TConstArrayView<T> InResult, T InBlendingFactor, TConstArrayView<T> InLerpSource1, TConstArrayView<T> InLerpSource2, TConstArrayView<T> InLerpSource3, TConstArrayView<T> InLerpSource4)
	{
		bool bGenerateReferences = CVarCameraCalibrationGenerateTestReferences.GetValueOnAnyThread();
		
		if (bGenerateReferences)
		{
			// Save current results as reference
			if (SaveTestReference(TestName, SectionName, InResult))
			{
				Test.AddInfo(FString::Printf(TEXT("Generated reference data for %s::%s"), *TestName, *SectionName));
			}
			else
			{
				Test.AddError(FString::Printf(TEXT("Failed to save reference data for %s::%s"), *TestName, *SectionName));
			}
		}
		else
		{
			// Load and compare against reference
			TArray<T> ReferenceValues;
			if (LoadTestReference(TestName, SectionName, ReferenceValues))
			{
				Test.TestEqual(FString::Printf(TEXT("%s::%s array size"), *TestName, *SectionName), InResult.Num(), ReferenceValues.Num());
				
				int32 NumToCheck = FMath::Min(InResult.Num(), ReferenceValues.Num());
				for(int32 Index = 0; Index < NumToCheck; ++Index)
				{
					Test.TestEqual(FString::Printf(TEXT("%s::%s[%d]"), *TestName, *SectionName, Index), InResult[Index], ReferenceValues[Index]);
				}
			}
			else
			{
				Test.AddError(FString::Printf(TEXT("Failed to load reference data for %s::%s. Run with CameraCalibrationCore.GenerateTestReferences=1 to generate reference data."), *TestName, *SectionName));
			}
		}
	}

	/** Enhanced single curve test that uses reference files when CVar is enabled */
	template<typename T>
	void TestSingleCurveEvaluationResultWithReference(FAutomationTestBase& Test, const FString& TestName, const FString& SectionName, TConstArrayView<T> InResult, T InBlendingFactor, TConstArrayView<T> InLerpSource1, TConstArrayView<T> InLerpSource2)
	{
		const bool bGenerateReferences = CVarCameraCalibrationGenerateTestReferences.GetValueOnAnyThread();
		
		if (bGenerateReferences)
		{
			// Save current results as reference
			if (SaveTestReference(TestName, SectionName, InResult))
			{
				Test.AddInfo(FString::Printf(TEXT("Generated reference data for %s::%s"), *TestName, *SectionName));
			}
			else
			{
				Test.AddError(FString::Printf(TEXT("Failed to save reference data for %s::%s"), *TestName, *SectionName));
			}
		}
		else
		{
			// Load and compare against reference
			TArray<T> ReferenceValues;
			if (LoadTestReference(TestName, SectionName, ReferenceValues))
			{
				Test.TestEqual(FString::Printf(TEXT("%s::%s array size"), *TestName, *SectionName), InResult.Num(), ReferenceValues.Num());
				
				int32 NumToCheck = FMath::Min(InResult.Num(), ReferenceValues.Num());
				for(int32 Index = 0; Index < NumToCheck; ++Index)
				{
					Test.TestEqual(FString::Printf(TEXT("%s::%s[%d]"), *TestName, *SectionName, Index), InResult[Index], ReferenceValues[Index]);
				}
			}
			else
			{
				Test.AddError(FString::Printf(TEXT("Failed to load reference data for %s::%s. Run with CameraCalibrationCore.GenerateTestReferences=1 to generate reference data."), *TestName, *SectionName));
			}
		}
	}

	void TestDistortionParameterCurveBlending(FAutomationTestBase& Test)
	{
		UWorld* ValidWorld = GetFirstWorld();
		if(ValidWorld == nullptr)
		{
			return;
		}
		
		// Create LensFile container
		const TCHAR* LensFileName = TEXT("AutomationTestLensFile");
		ULensFile* LensFile = NewObject<ULensFile>(GetTransientPackage(), LensFileName);
		
		// Set default lens info like BuildLensFileFromCalibrationData does
		LensFile->LensInfo.SensorDimensions = FVector2D(36.0f, 20.25f);

		const TCHAR* HandlerName = TEXT("AutomationTestLensHandler");
		USphericalLensDistortionModelHandler* ProducedLensDistortionHandler = NewObject<USphericalLensDistortionModelHandler>(ValidWorld, HandlerName, RF_Transient);
		ensure(ProducedLensDistortionHandler);

		if(ProducedLensDistortionHandler == nullptr)
		{
			return;
		}
		
		const bool bGenerateTestSuite = CVarCameraCalibrationGenerateTestReferences.GetValueOnAnyThread();
		
		constexpr int32 InputCount = 4;
		FLensData InputData[InputCount] =
		{
			// Input0
			FLensData(
				  0.0f,                                               // focus
				  0.0f,                                               // zoom
				  {{TArray<float>({1.0f, -1.0f, 0.0f, 0.0f, 0.0f})}}, // distortion
				  {FVector2D(8.0f, 7.0f)},                            // image center
				  {FVector2D(3.0f, 2.0f)}                             // focal length
			  ),

			// Input1
			FLensData(
				0.0f,                                                 // focus
				1.0f, 												  // zoom
				{{TArray<float>({2.0f, 0.0f, 0.0f, 0.0f, 1.0f})}},    // distortion
				{FVector2D(9.0f, 8.0f)}, 							  // image center
				{FVector2D(5.0f, 3.0f)}								  // focal length
			),

			// Input2
			FLensData(
				1.0f,                                                 // focus 
				0.0f, 												  // zoom
				{{TArray<float>({0.0f, 1.0f, 0.0f, 0.0f, -2.0f})}},   // distortion
				{FVector2D(2.0f, 4.0f)}, 							  // image center
				{FVector2D(4.0f, 6.0f)}								  // focal length
			),

			// Input3
			FLensData(
				1.0f,                                                 // focus
				1.0f, 												  // zoom
				{{TArray<float>({1.0f, 2.0f, 0.0f, 0.0f, 0.0f})}},	  // distortion
				{FVector2D(4.0f, 7.0f)},							  // image center
				{FVector2D(9.0f, 8.0f)}								  // focal length
			)
		};

		// Convert to TArray for easier handling
		TArray<FLensData> InputDataArray;
		for(int32 InputIndex = 0; InputIndex < InputCount; ++InputIndex)
		{
			//straight tests
			const FLensData& Input = InputData[InputIndex];
			LensFile->AddDistortionPoint(Input.Focus, Input.Zoom, Input.Distortion, Input.FocalLength);
			LensFile->AddImageCenterPoint(Input.Focus, Input.Zoom, Input.ImageCenter);
			InputDataArray.Add(Input);
		}
		
		if (bGenerateTestSuite)
		{
			// Generate comprehensive test suite with all test cases
			GenerateTestSuite(Test, TEXT("DistortionParameterCurveBlending"), LensFile, ProducedLensDistortionHandler, InputDataArray);
			return;
		}
		else
		{
			// Run tests from JSON test suite
			RunTestSuite(Test, TEXT("DistortionParameterCurveBlending"), LensFile, ProducedLensDistortionHandler);
			return;
		}
	}

	void TestLensFileAddPoints(FAutomationTestBase& Test)
	{
		// Create LensFile container
		const TCHAR* LensFileName = TEXT("AddPointsTestLensFile");
		ULensFile* LensFile = NewObject<ULensFile>(GetTransientPackage(), LensFileName);

		FDistortionInfo TestDistortionParams;
		TestDistortionParams.Parameters = TArray<float>({ 0.0f, 0.0f, 0.0f, 0.0f, 0.0f });

		FFocalLengthInfo TestFocalLength;
		TestFocalLength.FxFy = FVector2D(1.0f, 1.777f);

		TArray<FDistortionFocusPoint>& FocusPoints = LensFile->DistortionTable.GetFocusPoints();

		// Clear the LensFile's distortion table and confirm that the FocusPoints array is empty
		LensFile->ClearAll();
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 0);

		// Add a single point at F=0 Z=0 and confirm that FocusPoints array has one curve, and that curve has one ZoomPoint
		LensFile->AddDistortionPoint(0.0f, 0.0f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 1);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 1);

		// Add a point at F=0 Z=0.5 and confirm that FocusPoints array has one curve, and that curve has two ZoomPoints
		LensFile->AddDistortionPoint(0.0f, 0.5f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 1);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 2);

		// Attempt to add a duplicate point at F=0 Z=0.5 and confirm that FocusPoints array has one curve, and that curve has two ZoomPoints
		LensFile->AddDistortionPoint(0.0f, 0.5f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 1);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 2);

		// Test tolerance when adding a new zoom point
		LensFile->AddDistortionPoint(0.0f, 0.49f, TestDistortionParams, TestFocalLength);
		LensFile->AddDistortionPoint(0.0f, 0.4999f, TestDistortionParams, TestFocalLength);
		LensFile->AddDistortionPoint(0.0f, 0.5001f, TestDistortionParams, TestFocalLength);
		LensFile->AddDistortionPoint(0.0f, 0.51f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 1);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 4);

		// Add two points at F=1 Z=0 and F=1 Z=0.5 and confirm that FocusPoints array has two curves, and that each curve has two ZoomPoints
		LensFile->AddDistortionPoint(1.0f, 0.0f, TestDistortionParams, TestFocalLength);
		LensFile->AddDistortionPoint(1.0f, 0.5f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 2);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 4);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[1].ZoomPoints.Num(), 2);

		// Test sorting when adding a new focus point
		LensFile->AddDistortionPoint(0.5f, 0.0f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 3);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 4);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[1].ZoomPoints.Num(), 1);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[2].ZoomPoints.Num(), 2);

		// Test tolerance when adding focus points with slight differences in value
		LensFile->AddDistortionPoint(0.5001f, 0.25f, TestDistortionParams, TestFocalLength);
		LensFile->AddDistortionPoint(0.4999f, 0.5f, TestDistortionParams, TestFocalLength);
		Test.TestEqual(FString::Printf(TEXT("Num Focus Points")), FocusPoints.Num(), 3);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[0].ZoomPoints.Num(), 4);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[1].ZoomPoints.Num(), 3);
		Test.TestEqual(FString::Printf(TEXT("Num Zoom Points")), FocusPoints[2].ZoomPoints.Num(), 2);

		// Finally, test final state of each focus curve to ensure proper values and sorting
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[0]")), FocusPoints[0].ZoomPoints[0].Zoom, 0.0f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[1]")), FocusPoints[0].ZoomPoints[1].Zoom, 0.49f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[2]")), FocusPoints[0].ZoomPoints[2].Zoom, 0.5f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[3]")), FocusPoints[0].ZoomPoints[3].Zoom, 0.51f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[0]")), FocusPoints[1].ZoomPoints[0].Zoom, 0.0f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[1]")), FocusPoints[1].ZoomPoints[1].Zoom, 0.25f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[2]")), FocusPoints[1].ZoomPoints[2].Zoom, 0.5f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[0]")), FocusPoints[2].ZoomPoints[0].Zoom, 0.0f);
		Test.TestEqual(FString::Printf(TEXT("FocusPoints[0], ZoomPoints[1]")), FocusPoints[2].ZoomPoints[1].Zoom, 0.5f);
	}

	template <bool Const, typename Class, typename FuncType>
    struct TClassFunPtrType;
    
    template <typename Class, typename RetType, typename... ArgTypes>
    struct TClassFunPtrType<false, Class, RetType(ArgTypes...)>
    {
    	typedef RetType (Class::* Type)(ArgTypes...);
    };
    
    template <typename Class, typename RetType, typename... ArgTypes>
    struct TClassFunPtrType<true, Class, RetType(ArgTypes...)>
    {
    	typedef RetType (Class::* Type)(ArgTypes...) const;
    };

	template <typename FPointType>
	void TestRemoveSinglePoint(FAutomationTestBase& Test, ULensFile* InLensFile, ELensDataCategory InDataCategory,
	                       typename TClassFunPtrType<false, ULensFile, void(float, float, const FPointType&)>::Type
	                       InFunc)
	{
		UEnum* Enum = StaticEnum<ELensDataCategory>();
		const FString EnumString = Enum ? Enum->GetNameByIndex(static_cast<uint8>(InDataCategory)).ToString() : "";
		
		Test.AddInfo(FString::Printf(TEXT("Remove %s"), *EnumString));
		const FPointType PointInfo;
		const float FocusValue = 0.12f;
		const float ZoomValue = 0.91f;
		(InLensFile->*InFunc)(FocusValue, ZoomValue, PointInfo);
		InLensFile->RemoveZoomPoint(InDataCategory, FocusValue, ZoomValue);
		const int32 PointsNum = InLensFile->GetTotalPointNum(InDataCategory);
		Test.TestEqual(TEXT("Num Zoom Points"), PointsNum, 0);
	}

	void TestLensFileRemovePoints(FAutomationTestBase& Test)
	{
		constexpr auto LensFileName = TEXT("RemovePointsTestLensFile");
		ULensFile* LensFile = NewObject<ULensFile>(GetTransientPackage(), LensFileName);

		Test.AddInfo(TEXT("Remove ELensDataCategory::Focus"));
		{
			constexpr float FocusKeyTimeValue = 0.12f;
			constexpr float FocusKeyMappingValue = 0.91f;
			LensFile->EncodersTable.Focus.AddKey(FocusKeyTimeValue, FocusKeyMappingValue);
			Test.TestEqual(TEXT("Time Value Should be the same"), LensFile->EncodersTable.GetFocusInput(0), FocusKeyTimeValue);
			LensFile->EncodersTable.RemoveFocusPoint(FocusKeyTimeValue);
			Test.TestEqual(TEXT("Num Focus Points"), LensFile->EncodersTable.GetNumFocusPoints(), 0);
		}

		Test.AddInfo(TEXT("Remove ELensDataCategory::Iris"));
		{
			constexpr float IrisKeyTimeValue = 0.12f;
			constexpr float IrisKeyMappingValue = 0.91f;
			LensFile->EncodersTable.Iris.AddKey(IrisKeyTimeValue, IrisKeyMappingValue);
			Test.TestEqual(TEXT("Time Value should be the same"), LensFile->EncodersTable.GetIrisInput(0), IrisKeyTimeValue);
			LensFile->EncodersTable.RemoveIrisPoint(IrisKeyTimeValue);
			Test.TestEqual(TEXT("Num Iris Points"), LensFile->EncodersTable.GetNumIrisPoints(), 0);
		}
		
		Test.AddInfo(TEXT("Remove all point ELensDataCategory::Zoom"));
		{
			const FFocalLengthInfo FocalLengthInfo;
			const float FocusValue = 0.12f;
			const float ZoomValue1 = 0.191f;
			const float ZoomValue2 = 0.91f;
			LensFile->AddFocalLengthPoint(FocusValue, ZoomValue1, FocalLengthInfo);
			LensFile->AddFocalLengthPoint(FocusValue, ZoomValue2, FocalLengthInfo);
			LensFile->RemoveFocusPoint(ELensDataCategory::Zoom, FocusValue);
			const int32 PointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num Zoom Points"), PointsNum, 0);
		}

		Test.AddInfo(TEXT("Remove ELensDataCategory::Distortion"));
		{
			const FFocalLengthInfo FocalLengthInfo;
			const FDistortionInfo DistortionPoint;
			const float FocusValue = 0.12f;
			const float ZoomValue = 0.191f;
			LensFile->AddDistortionPoint(FocusValue, ZoomValue, DistortionPoint, FocalLengthInfo);
			LensFile->RemoveZoomPoint(ELensDataCategory::Distortion, FocusValue, ZoomValue);
			const int32 DistortionPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Distortion);
			int32 FocalLengthPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num Distortion Points"), DistortionPointsNum, 0);
			Test.TestEqual(TEXT("Num FocalLength Points"), FocalLengthPointsNum, 1);
			LensFile->RemoveZoomPoint(ELensDataCategory::Zoom, FocusValue, ZoomValue);
			FocalLengthPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num FocalLength Points"), FocalLengthPointsNum, 0);
		}

		Test.AddInfo(TEXT("Remove ELensDataCategory::Distortion, add focal length before add distortion point"));
		{
			const FFocalLengthInfo FocalLengthInfo;
			const FDistortionInfo DistortionPoint;
			const float FocusValue = 0.12f;
			const float ZoomValue = 0.191f;
			LensFile->AddFocalLengthPoint(FocusValue, ZoomValue, FocalLengthInfo);
			LensFile->AddDistortionPoint(FocusValue, ZoomValue, DistortionPoint, FocalLengthInfo);
			int32 FocalLengthPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num FocalLength Points"), FocalLengthPointsNum, 1);
			LensFile->RemoveZoomPoint(ELensDataCategory::Distortion, FocusValue, ZoomValue);
			const int32 DistortionPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Distortion);
			FocalLengthPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num Distortion Points"), DistortionPointsNum, 0);
			Test.TestEqual(TEXT("Num FocalLength Points"), FocalLengthPointsNum, 1);
			LensFile->RemoveZoomPoint(ELensDataCategory::Zoom, FocusValue, ZoomValue);
			FocalLengthPointsNum = LensFile->GetTotalPointNum(ELensDataCategory::Zoom);
			Test.TestEqual(TEXT("Num FocalLength Points"), FocalLengthPointsNum, 0);
		}

		TestRemoveSinglePoint<FFocalLengthInfo>(Test, LensFile, ELensDataCategory::Zoom, &ULensFile::AddFocalLengthPoint);
		TestRemoveSinglePoint<FImageCenterInfo>(Test, LensFile, ELensDataCategory::ImageCenter, &ULensFile::AddImageCenterPoint);
		TestRemoveSinglePoint<FNodalPointOffset>(Test, LensFile, ELensDataCategory::NodalOffset, &ULensFile::AddNodalOffsetPoint);
		TestRemoveSinglePoint<FSTMapInfo>(Test, LensFile, ELensDataCategory::STMap, &ULensFile::AddSTMapPoint);
	}
}

DEFINE_SPEC(FTestCameraCalibrationCore, "Plugins.CameraCalibrationCore", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FTestCameraCalibrationCore::Define()
{
	It("DistortionParameterCurveBlending", [this]()
		{
			CameraCalibrationTestUtil::TestDistortionParameterCurveBlending(*this);
		});

	It("LensFileAddPoints", [this]()
		{
			CameraCalibrationTestUtil::TestLensFileAddPoints(*this);
		});

	It("LensFileRemovePoints", [this]()
		{
			CameraCalibrationTestUtil::TestLensFileRemovePoints(*this);
		});
}

#endif // WITH_DEV_AUTOMATION_TESTS



