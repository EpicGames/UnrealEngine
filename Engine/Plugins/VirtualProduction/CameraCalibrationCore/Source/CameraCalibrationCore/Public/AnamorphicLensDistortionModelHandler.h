// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensDistortionModelHandlerBase.h"

#include "Models/AnamorphicLensModel.h"

#include "AnamorphicLensDistortionModelHandler.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/** Lens distortion handler for an Anamorphic lens model that implements the 3DE4 Anamorphic - Standard Degree 4 model */
UCLASS(MinimalAPI, BlueprintType)
class UAnamorphicLensDistortionModelHandler : public ULensDistortionModelHandlerBase
{
	GENERATED_BODY()

protected:
	//~ Begin ULensDistortionModelHandlerBase interface
	UE_API virtual void InitializeHandler() override;
	UE_API virtual FVector2D ComputeDistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual FVector2D ComputeUndistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual void InitDistortionMaterials() override;
	UE_API virtual void UpdateMaterialParameters() override;
	UE_API virtual void InterpretDistortionParameters() override;
	UE_API virtual FString GetDistortionShaderPath() const override;
	UE_API virtual void ExecuteDistortionShader(class FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const override;
	virtual bool IsForwardDistorting() const override { return false; }
	//~ End ULensDistortionModelHandlerBase interface

private:
	/** Anamorphic lens distortion parameters */
	FAnamorphicDistortionParameters AnamorphicParameters;
};

#undef UE_API
