// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BrownConradyUDLensDistortionModelHandler.h"

#include "BrownConradyDULensDistortionModelHandler.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/** 
 * Lens distortion handler for a Brown-Conrady D-U lens model 
 * This model interprets the distortion parameters in the reverse direction (distorted to undistorted)
 * compared to the U-D model. It subclasses the U-D handler and overrides key functions to swap directions.
 */
UCLASS(MinimalAPI, BlueprintType)
class UBrownConradyDULensDistortionModelHandler : public UBrownConradyUDLensDistortionModelHandler
{
	GENERATED_BODY()

protected:
	//~ Begin ULensDistortionModelHandlerBase interface
	UE_API virtual void InitializeHandler() override;
	virtual bool IsForwardDistorting() const override { return false; }
	UE_API virtual FVector2D ComputeDistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual FVector2D ComputeUndistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual void InitDistortionMaterials() override;
	//~ End ULensDistortionModelHandlerBase interface
};

#undef UE_API
