// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensDistortionModelHandlerBase.h"

#include "Models/BrownConradyUDLensModel.h"

#include "BrownConradyUDLensDistortionModelHandler.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/** Lens distortion handler for a Brown-Conrady U-D lens model that implements the Brown-Conrady polynomial model */
UCLASS(MinimalAPI, BlueprintType)
class UBrownConradyUDLensDistortionModelHandler : public ULensDistortionModelHandlerBase
{
	GENERATED_BODY()

protected:
	//~ Begin ULensDistortionModelHandlerBase interface
	UE_API virtual void InitializeHandler() override;
	UE_API virtual FVector2D ComputeDistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual FVector2D ComputeUndistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual void InitDistortionMaterials() override;
	UE_API virtual void UpdateMaterialParameters() override;
	UE_API virtual void InterpretDistortionParameters() override;
	UE_API virtual FString GetDistortionShaderPath() const override;
	UE_API virtual void ExecuteDistortionShader(class FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const override;
	//~ End ULensDistortionModelHandlerBase interface

private:
	/** Brown-Conrady U-D lens distortion parameters (k1, k2, k3, k4, k5, k6, p1, p2) */
	FBrownConradyUDDistortionParameters BrownConradyUDParameters;
};

#undef UE_API