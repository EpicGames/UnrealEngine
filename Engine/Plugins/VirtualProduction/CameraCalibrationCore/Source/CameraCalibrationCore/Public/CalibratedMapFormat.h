// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CalibratedMapFormat.generated.h"

/** Specifies the location of the pixel origin in a calibrated map */
UENUM()
enum class ECalibratedMapPixelOrigin : uint8
{
	TopLeft,
	BottomLeft,

	MAX // Required option for shader permutation
};

/** Specifies which two channels contain the calibrated map data (or None if there is no data) */
UENUM()
enum class ECalibratedMapChannels : uint8
{
	RG,
	BA,
	None,

	MAX // Required option for shader permutation
};

/** Formatting options for processing a calibrated map */
USTRUCT()
struct FCalibratedMapFormat
{
	GENERATED_BODY()

public:
	/** Specifies where in the image the (0, 0) pixel is */
	UPROPERTY(EditAnywhere, Category="Format")
	ECalibratedMapPixelOrigin PixelOrigin = ECalibratedMapPixelOrigin::TopLeft;

	/** Specifies which two channels contain the undistortion map (or None if there is no undistortion data) */
	UPROPERTY(EditAnywhere, Category = "Format")
	ECalibratedMapChannels UndistortionChannels = ECalibratedMapChannels::RG;

	/** Specifies which two channels contain the distortion map (or None if there is no distortion data) */
	UPROPERTY(EditAnywhere, Category = "Format")
	ECalibratedMapChannels DistortionChannels = ECalibratedMapChannels::BA;
};
