// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProceduralMeshComponent.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"

#include "CalibrationPointComponent.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API


/** Ways of visualizing the calibration points */
UENUM(BlueprintType)
enum ECalibrationPointVisualization : int
{
	CalibrationPointVisualizationCube    UMETA(DisplayName = "Cubes"),
	CalibrationPointVisualizationPyramid UMETA(DisplayName = "Pyramids"),
};

/**
 * One or more instances of this component can be added to an actor (e.g. a static mesh actor blueprint), 
 * and should be placed at geometrically and visually distinct landmarks of the object.
 * These 3d points will then be optionally used by any given nodal offset tool implementation to
 * make a 3d-2d correspondence with the 2d points detected in the live action media.
 */
UCLASS(MinimalAPI, ClassGroup = (Calibration), meta = (BlueprintSpawnableComponent), meta = (DisplayName = "Calibration Point"))
class UCalibrationPointComponent : public UProceduralMeshComponent
{
	GENERATED_BODY()

public:

	UE_API UCalibrationPointComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin UActorComponent Interface
	UE_API virtual void OnRegister() override;
	//~ End UActorComponent Interface

	//~ Begin UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

public:

	/** A way to group many points in a single component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Calibration")
	TMap<FString,FVector> SubPoints;

	/** Draws a visual representation of the calibration points */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	bool bVisualizePointsInEditor = false;

	/** Scales up/down the size of the point visualization meshes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	float PointVisualizationScale = 1.0f;

	/** Shape used to visualize the calibration (sub)points */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	TEnumAsByte<ECalibrationPointVisualization> VisualizationShape = ECalibrationPointVisualization::CalibrationPointVisualizationCube;

public:

	/** 
	 * Returns the World location of the subpoint (or the component) specified by name 
	 * 
	 * @param InPointName Name of the point or subpoint. If not namespaced the component name will have priority over subpoint name.
	 * @param OutLocation World location of the specified subpoint.
	 * 
	 * @return True if successful.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Calibration")
	UE_API bool GetWorldLocation(const FString& InPointName, FVector& OutLocation) const;

	/**
	 * Namespaces the given subpoint name. Does not check that the subpoint exists.
	 * 
	 * @param InSubpointName Name of the subpoint to namespace
	 * @param OutNamespacedName The output namespaced subpoint name
	 * 
	 * @return True if successful
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Calibration")
	UE_API bool NamespacedSubpointName(const FString& InSubpointName, FString& OutNamespacedName) const;

	/** 
	 * Gathers the namespaced names of the subpoints and the component itself.
	 * 
	 * @param OutNamespacedNames Array of names to be filled out by this function. Will not empty it.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Calibration")
	UE_API void GetNamespacedPointNames(TArray<FString>& OutNamespacedNames) const;

	/** Rebuilds the point visualization. */
	UFUNCTION(BlueprintCallable, Category = "Calibration")
	UE_API void RebuildVertices();
};

#undef UE_API
