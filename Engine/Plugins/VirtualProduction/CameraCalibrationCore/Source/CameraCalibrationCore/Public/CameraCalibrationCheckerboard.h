// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "GameFramework/Actor.h"

#include "CameraCalibrationCheckerboard.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

class UCalibrationPointComponent;
class UStaticMesh;
class UMaterialInterface;

/**
 * Dynamic checkerboad actor
 */
UCLASS(MinimalAPI)
class ACameraCalibrationCheckerboard : public AActor
{
	GENERATED_BODY()

public:

	UE_API ACameraCalibrationCheckerboard();

public:

	/** Rebuilds the instanced components that make up this checkerboard */
	UFUNCTION(BlueprintCallable, Category = "Calibration")
	UE_API void Rebuild();

	//~ Begin UObject Interface
	UE_API virtual void OnConstruction(const FTransform& Transform) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

private:
	/** Removes mesh components that make up the checkerboard */
	void ClearInstanceCheckeboardMeshComponents();

public:

	/** Root component, gives the Actor a transform */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<USceneComponent> Root;

	/** TopLeft calibration point */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UCalibrationPointComponent> TopLeft;

	/** TopRight calibration point */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UCalibrationPointComponent> TopRight;

	/** BottomLeft calibration point */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UCalibrationPointComponent> BottomLeft;

	/** BottomRight calibration point */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UCalibrationPointComponent> BottomRight;

	/** Center calibration point */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UCalibrationPointComponent> Center;


	/** Number of rows */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	int32 NumCornerRows = 2;

	/** Number of columns */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	int32 NumCornerCols = 2;

	/** Length of the side of each square */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	float SquareSideLength = 3.2089f;

	/** Thickness of checkerboard. Not used for calibration purposes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Calibration")
	float Thickness = 0.1f;

	/** The static mesh that we are going to use for all the cubes */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UStaticMesh> CubeMesh;

	/** The material that we are going to use for all the odd cubes */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UMaterialInterface> OddCubeMaterial;

	/** The material that we are going to use for all the even cubes */
	UPROPERTY(EditAnywhere, Category = "Calibration")
	TObjectPtr<UMaterialInterface> EvenCubeMaterial;
};

#undef UE_API
