// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "LensDistortionModelHandlerBase.h"
#include "LensFile.h"

#include "CameraCalibrationSettings.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisplacementMapResolutionChanged, const FIntPoint NewDisplacementMapResolution);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCalibrationInputToleranceChanged, const float NewTolerance);
#endif

/**
 * Settings for the CameraCalibration plugin modules. 
 */
UCLASS(MinimalAPI, config=Game)
class UCameraCalibrationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UCameraCalibrationSettings();

	//~ Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/** Get the list of calibration overlay override names */
	UE_API TArray<FName> GetCalibrationOverlayMaterialOverrideNames() const;

	/** Get the override MaterialInterface associated with the input overlay name */
	UE_API UMaterialInterface* GetCalibrationOverlayMaterialOverride(const FName OverlayName) const;

	/** Gets a multicast delegate which is called whenever the displacement map resolution project setting changes */
	UE_API FOnDisplacementMapResolutionChanged& OnDisplacementMapResolutionChanged();

	/** Gets a multicast delegate which is called whenever the displacement map resolution project setting changes */
	UE_API FOnCalibrationInputToleranceChanged& OnCalibrationInputToleranceChanged();
#endif

public:

	/**
	 * Get the default startup lens file.
	 *
	 * @return The lens file, or nullptr if not set.
	 */
	UE_API ULensFile* GetStartupLensFile() const;

	/** Get the resolution that should be used for distortion and undistortion displacement maps */
	FIntPoint GetDisplacementMapResolution() const { return DisplacementMapResolution; }

	/** Get the tolerance to use when adding or accessing data in a calibrated LensFile */
	float GetCalibrationInputTolerance() const { return CalibrationInputTolerance; }

	/** Get the length of time (in seconds) the calibration tool will wait when attempting to detect a checkerboard pattern in the media image */
	float GetCheckerboardDetectionTimeout() const { return CheckerboardDetectionTimeout; }

	/** Get the default MaterialInterface used by the input Model Handler class to write the undistortion displacement map */
	UE_API UMaterialInterface* GetDefaultUndistortionDisplacementMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const;

	/** Get the default MaterialInterface used by the input Model Handler class to write the distortion displacement map */
	UE_API UMaterialInterface* GetDefaultDistortionDisplacementMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const;

	/** Get the default MaterialInterface used by the input Model Handler class to apply the post-process lens distortion effect */
	UE_API UMaterialInterface* GetDefaultDistortionMaterial(const TSubclassOf<ULensDistortionModelHandlerBase>& InModelHandler) const;

	/** Returns true if the calibration dataset import and export featuers are enabled, false otherwise */
	bool IsCalibrationDatasetImportExportEnabled() const { return bEnableCalibrationDatasetImportExport; }

	/** Returns the scene view extension priority ranking. */
	int32 GetSceneViewExtensionPriority() const { return SceneViewExtensionPriority; }

#if WITH_EDITOR
protected:
	FOnDisplacementMapResolutionChanged DisplacementMapResolutionChangedDelegate;
	FOnCalibrationInputToleranceChanged CalibrationInputToleranceChangedDelegate;
#endif // WITH_EDITOR

private:
	/** 
	 * Startup lens file for the project 
	 * Can be overriden. Priority of operation is
	 * 1. Apply startup lens file found in 'CameraCalibration.StartupLensFile' cvar at launch
	 * 2. If none found, apply user startup file (only for editor runs)
	 * 3. If none found, apply projet startup file (this one)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings", meta = (ConfigRestartRequired = true))
	TSoftObjectPtr<ULensFile> StartupLensFile;

	/** Resolution used when creating new distortion and undistortion displacement maps */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	FIntPoint DisplacementMapResolution = FIntPoint(256, 256);

	/** Tolerance to use when adding or accessing data in a calibrated LensFile */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	float CalibrationInputTolerance = 0.001f;

	/** Length of time (in seconds) the calibration tool will wait when attempting to detect a checkerboard pattern in the media image */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	float CheckerboardDetectionTimeout = 2.0f;

	/** Map of Lens Distortion Model Handler classes to the default displacement map material used by that class */
	UPROPERTY(config)
	TMap<TSubclassOf<ULensDistortionModelHandlerBase>, TSoftObjectPtr<UMaterialInterface>> DefaultUndistortionDisplacementMaterials;

	/** Map of Lens Distortion Model Handler classes to the default displacement map material used by that class */
	UPROPERTY(config)
	TMap<TSubclassOf<ULensDistortionModelHandlerBase>, TSoftObjectPtr<UMaterialInterface>> DefaultDistortionDisplacementMaterials;

	/** Map of Lens Distortion Model Handler classes to the default lens distortion post-process material used by that class */
	UPROPERTY(config)
	TMap<TSubclassOf<ULensDistortionModelHandlerBase>, TSoftObjectPtr<UMaterialInterface>> DefaultDistortionMaterials;

#if WITH_EDITORONLY_DATA
	/** Map of overlay names to override overlay materials */
	UPROPERTY(config, EditAnywhere, Category = "Overlays")
	TMap<FName, TSoftObjectPtr<UMaterialInterface>> CalibrationOverlayMaterialOverrides;
#endif

	/** Setting to toggle the calibration dataset import and export features */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bEnableCalibrationDatasetImportExport = true;

	/** Lens distortion (scene view extension) post-processing priority, which defaults to before OpenColorIO. */
	UPROPERTY(config, EditAnywhere, Category = Settings, AdvancedDisplay)
	int32 SceneViewExtensionPriority;

private:
	/** Delegate handle to run after the engine is initialized */
	FDelegateHandle PostEngineInitHandle;
};

/**
* Lens Data Table Editor Category color. Using for the color of the curves
*/
USTRUCT()
struct FLensDataCategoryEditorColor
{
	GENERATED_BODY()

	/** Get the color for specific category */
	FColor GetColorForCategory(const ELensDataCategory InCategory) const
	{
		switch (InCategory) {
		case ELensDataCategory::Focus:
			return Focus;
		case ELensDataCategory::Iris:
			return Iris;
		case ELensDataCategory::Zoom:
			return Zoom;
		case ELensDataCategory::Distortion:
			return Distortion;
		case ELensDataCategory::ImageCenter:
			return ImageCenter;
		case ELensDataCategory::STMap:
			return STMap;
		case ELensDataCategory::NodalOffset:
			return NodalOffset;
		default:
			checkNoEntry();
			return FColor::Black;
		}
	}

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor Focus = FColor::Red;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor Iris = FColor::Green;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor Zoom = FColor::Blue;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor Distortion = FColor::Cyan;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor ImageCenter = FColor::Yellow;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor STMap = FColor::Orange;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FColor NodalOffset = FColor::Purple;
};

/** Units used to display/interpret Focal Length and Image Center */
UENUM()
enum class ELensDisplayUnit : uint8
{
	Millimeters,
	Pixels,
	Normalized
};

/**
 * Settings for the camera calibration when in editor and standalone.
 * @note Cooked games don't use this setting.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UCameraCalibrationEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UCameraCalibrationEditorSettings();

	//~ Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const;
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface

public:

#if WITH_EDITORONLY_DATA

	/**
	 * True if a lens file button shortcut should be added to level editor toolbar.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings", Meta = (ConfigRestartRequired = true, DisplayName = "Enable Lens File Toolbar Button"))
	bool bShowEditorToolbarButton = false;

	/**
	 * Data Table category color settings
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	FLensDataCategoryEditorColor CategoryColor;

	/**
	 * Time slider enable flag
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings", Meta = (ToolTip = "Enable or Disable Time input driven by evaluation inputs."))
	bool bEnableTimeSlider = true;

	/** 
	 * Units used to display/interpret Focal Length and Image Center 
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	ELensDisplayUnit DefaultDisplayUnit = ELensDisplayUnit::Millimeters;

	/**
	 * If true, the media player in the calibration tools will always use the default step rate. 
	 * Otherwise, it will try to use the frame rate of the media to step by exactly one frame. 
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	bool bForceDefaultMediaStepRate = false;

	/**
	 * The default step rate (ms) that the media player in the calibration tools should use when stepping forward/back
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	float DefaultMediaStepRateInMilliseconds = 100.0f;

private:

	/** 
	 * Startup lens file per user in editor 
	 * Can be overridden. Priority of operation is
	 * 1. Apply startup lens file found in 'CameraCalibration.StartupLensFile' cvar at launch
	 * 2. If none found, apply user startup file (this one)
	 * 3. If none found, apply project startup file
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings")
	TSoftObjectPtr<ULensFile> UserLensFile;
#endif

public:

	/**
	 * Get the lens file used by the engine when in the editor and standalone.
	 *
	 * @return The lens file, or nullptr if not set.
	 */
	UE_API ULensFile* GetUserLensFile() const;

	/** Set the lens file used by the engine when in the editor and standalone. */
	UE_API void SetUserLensFile(ULensFile* InLensFile);
};

#undef UE_API
