// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SBoxPanel.h"

#include "CameraCalibrationStep.generated.h"

struct FGeometry;
struct FKey;
struct FPointerEvent;

class FCameraCalibrationStepsController;
class UMaterialInstanceDynamic;

/**
 * Interface of a camera calibration step. These will appear in a Camera Calibration Toolkit tab.
 */
UCLASS(MinimalAPI, Abstract)
class UCameraCalibrationStep : public UObject
{
	GENERATED_BODY()

public:

	/** Make sure you initialize before using the object */
	virtual void Initialize(TWeakPtr<FCameraCalibrationStepsController> InCameraCalibrationStepController) {};

	/** Clean up resources and don't use CameraCalibrationStepController anymore */
	virtual void Shutdown() {};

	/** Called every frame */
	virtual void Tick(float DeltaTime) {};

	/** Callback when viewport is clicked. Returns false if the event was not handled. */
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false;  };

	/** Callback when key is pressed while viewport is focused. Returns false if the event was not handled. */
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) { return false; };

	/** Callback when the user performs a marquee select operation. Returns false if the event was not handled. */
	virtual bool OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition) { return false; };

	/** Returns the UI of this camera calibration step. Expected to only be called once */
	virtual TSharedRef<SWidget> BuildUI() { return SNew(SVerticalBox); };

	/** Returns a title or friendly name that can be placed in UI */
	virtual FName FriendlyName() const { return TEXT("Invalid"); };

	/** Returns true if the given calibration step is a known prerequisite for this step */
	virtual bool DependsOnStep(UCameraCalibrationStep* Step) const { return false; };

	/** Returns the overlay MID used by this step */
	virtual UMaterialInstanceDynamic* GetOverlayMID() const { return nullptr; };

	/** Returns true if this step has enabled its overlay */
	virtual bool IsOverlayEnabled() const { return false; };

	/** Called when this step is the active step in the UI */
	virtual void Activate() {};

	/** Called when this step is no longer the active step in the UI */
	virtual void Deactivate() {};

	/** Returns true if the step is active */
	virtual bool IsActive() const { return false; };

	/** Returns the parent camera calibration steps controller */
	virtual FCameraCalibrationStepsController* GetCameraCalibrationStepsController() const { return nullptr; };
};
