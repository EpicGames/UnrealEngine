// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "CoreMinimal.h"

#include "CameraCalibrationTypes.h"
#include "CameraImageCenterAlgo.h"
#include "CameraNodalOffsetAlgo.h"
#include "CameraCalibrationStep.h"
#include "Containers/ArrayView.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensDistortionSceneViewExtension.h"
#include "LensFile.h"
#include "UObject/ObjectKey.h"

#include "CameraCalibrationSubsystem.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

class UCineCameraComponent;
class UTexture2D;
struct FCharucoBoardConfigShadow;

/** Delegate for generating Charuco board textures without OpenCV dependency */
DECLARE_DELEGATE_RetVal_ThreeParams(UTexture2D*, FGenerateCharucoBoardDelegate, const FCharucoBoardConfigShadow&, FIntPoint, int32);

/**
 * Camera Calibration subsystem
 */
UCLASS(MinimalAPI)
class UCameraCalibrationSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:

	/** Get the default lens file. */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API ULensFile* GetDefaultLensFile() const;

	/** Get the default lens file. */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API void SetDefaultLensFile(ULensFile* NewDefaultLensFile);

	/** Facilitator around the picker to get the desired lens file. */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API ULensFile* GetLensFile(const FLensFilePicker& Picker) const;

	UE_DEPRECATED(5.1, "This function has been deprecated. The subsystem no longer tracks distortion handlers. Query for a handler from a specific Lens Component belonging to the input camera.")
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TArray<ULensDistortionModelHandlerBase*> GetDistortionModelHandlers(UCineCameraComponent* Component);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "This function has been deprecated. The subsystem no longer tracks distortion handlers. Query for a handler from a specific Lens Component belonging to the input camera.")
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API ULensDistortionModelHandlerBase* FindDistortionModelHandler(UPARAM(ref)FDistortionHandlerPicker& DistortionHandlerPicker, bool bUpdatePicker = true) const;

	UE_DEPRECATED(5.1, "This function has been deprecated. The subsystem no longer tracks distortion handlers. Query for a handler from a specific Lens Component belonging to the input camera.")
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API ULensDistortionModelHandlerBase* FindOrCreateDistortionModelHandler(UPARAM(ref)FDistortionHandlerPicker& DistortionHandlerPicker, const TSubclassOf<ULensModel> LensModelClass);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.1, "This function has been deprecated. The subsystem no longer tracks distortion handlers. Query for a handler from a specific Lens Component belonging to the input camera.")
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API void UnregisterDistortionModelHandler(UCineCameraComponent* Component, ULensDistortionModelHandlerBase* Handler);

	/** Return the ULensModel subclass that was registered with the input model name */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TSubclassOf<ULensModel> GetRegisteredLensModel(FName ModelName) const;

	/** Returns the nodal offset algorithm by name */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TSubclassOf<UCameraNodalOffsetAlgo> GetCameraNodalOffsetAlgo(FName Name) const;

	/** Returns an array with the names of the available nodal offset algorithms */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TArray<FName> GetCameraNodalOffsetAlgos() const;

	/** Returns the image center algorithm by name */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TSubclassOf<UCameraImageCenterAlgo> GetCameraImageCenterAlgo(FName Name) const;

	/** Returns an array with the names of the available image center algorithms */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TArray<FName> GetCameraImageCenterAlgos() const;

	/** Returns the overlay material associated with the input overlay name */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API UMaterialInterface* GetOverlayMaterial(const FName& OverlayName) const;

	/** 
	 * Returns a list of all overlays known to the subsystem
	 * This includes the default overlays listed in the camera calibration settings 
	 * as well as any of overlays that have been registered with this subsystem
	 */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TArray<FName> GetOverlayMaterialNames() const;

	/** Returns an array with the names of the available camera calibration steps */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TArray<FName> GetCameraCalibrationSteps() const;

	/** Returns the camera calibration step by name */
	UFUNCTION(BlueprintCallable, Category = "Lens Distortion")
	UE_API TSubclassOf<UCameraCalibrationStep> GetCameraCalibrationStep(FName Name) const;

public:
	//~ Begin USubsystem interface
	UE_API virtual void Deinitialize() override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	//~ End USubsystem interface

	/** Add a new Lens Model to the registered model map */
	UE_API void RegisterDistortionModel(TSubclassOf<ULensModel> LensModel);

	/** Remove a Lens Model from the registered model map */
	UE_API void UnregisterDistortionModel(TSubclassOf<ULensModel> LensModel);

	/** Register a new overlay material name and path that can be queried from camera calibration tools */
	UE_API void RegisterOverlayMaterial(const FName& MaterialName, const FSoftObjectPath& MaterialPath);

	/** Unregister an overlay material */
	UE_API void UnregisterOverlayMaterial(const FName& MaterialName);

	/** Adds the input distortion state, rendering mode and blending parameters to the Lens Distortion Scene View Extension for the input camera actor. */
	UE_API void SetLensDistortionSVEState(ACameraActor* CameraActor, FDisplacementMapBlendingParams DistortionState, ULensDistortionModelHandlerBase* LensDistortionHandler, EDistortionRenderingMode InRenderingMode);

	/** Removes all distortion state from the Lens Distortion Scene View Extension for the input camera actor */
	UE_API void ClearLensDistortionSVEState(ACameraActor* CameraActor);

	/** Get lens distortion scene view extension. */
	UE_API const TSharedPtr<FLensDistortionSceneViewExtension, ESPMode::ThreadSafe>& GetSceneViewExtension() const;

	/** Set the delegate for generating Charuco board textures */
	UE_API void SetCharucoBoardTextureGenerator(FGenerateCharucoBoardDelegate InDelegate);

	/** Generate a Charuco board texture using the registered delegate (if available) */
	UE_API UTexture2D* GenerateCharucoBoardTexture(const FCharucoBoardConfigShadow& BoardConfig, FIntPoint ImageSize, int32 MarginSize = 10);

private:
	/** Default lens file to use when no override has been provided */
	UPROPERTY(Transient)
	TObjectPtr<ULensFile> DefaultLensFile = nullptr;

	/** Map of model names to ULensModel subclasses */
	UPROPERTY(Transient)
	TMap<FName, TSubclassOf<ULensModel>> LensModelMap;

	/** Holds the registered camera nodal offset algos */
	UPROPERTY(Transient)
	TMap<FName, TSubclassOf<UCameraNodalOffsetAlgo>> CameraNodalOffsetAlgosMap;

	/** Holds the registered camera image center algos */
	UPROPERTY(Transient)
	TMap<FName, TSubclassOf<UCameraImageCenterAlgo>> CameraImageCenterAlgosMap;

	/** Holds the registered camera calibration steps */
	UPROPERTY(Transient)
	TMap<FName, TSubclassOf<UCameraCalibrationStep>> CameraCalibrationStepsMap;

	/** Map of overlay names to overlay materials */
	TMap<FName, TSoftObjectPtr<UMaterialInterface>> RegisteredOverlayMaterials;

	/** Map of actor components to the authoritative lens model that should be used with that component */
	TMap<FObjectKey, TSubclassOf<ULensModel>> ComponentsWithAuthoritativeModels;

	/** Lens Distortion Scene View Extension, used to render distortion and undistortion displacement maps */
	TSharedPtr<FLensDistortionSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;

	/** Delegate for generating Charuco board textures without OpenCV dependency */
	FGenerateCharucoBoardDelegate CharucoBoardTextureGenerator;

private:

	FDelegateHandle PostEngineInitHandle;

};

#undef UE_API
