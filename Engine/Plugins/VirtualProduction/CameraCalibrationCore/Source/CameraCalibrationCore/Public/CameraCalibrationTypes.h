// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraComponent.h"
#include "LensData.h"
#include "Misc/Guid.h"

#include "CameraCalibrationTypes.generated.h"

/** Specifies how the distortion should be rendered in the post-processing pipeline */
UENUM(BlueprintType)
enum class EDistortionRenderingMode : uint8
{
	/** Use the lens model's preferred rendering mode */
	Preferred = 2u,

	/** Use the plugin post process material as a camera blendable. */
	PostProcessMaterial = 0u,

	/**
	  * Apply lens distortion inside the renderer's temporal super resolution.
	  * 
	  * Renderer passes after TSR are automatically made aware of lens distortion through renderer diplacements maps (FLensDistortionLUT).
	  * Note that if TSR is not active or if r.TSR.LensDistortion = 0, distortion will be applied later in PrimaryUpscale instead.
	  */
	TemporalSuperResolution = 1u,

	/**
	* Use the plugin post process material as a scene view extension pass after tone-mapping.
	* 
	* This can be used to apply distortion before other scene view extensions (i.e. CompositeCore, OpenColorIO),
	* which is otherwise not possible with post-process materials.
	*/
	PassAfterTonemap = 3u,

	//~ DEPRECATED
	SceneViewExtension UE_DEPRECATED(5.8, "Renamed to TemporalSuperResolution") = TemporalSuperResolution UMETA(Hidden),
};

/** Results from a distortion calibration, including camera intrinsics and either the parameters to an analytical model or an ST Map */
USTRUCT(BlueprintType)
struct FDistortionCalibrationResult
{
	GENERATED_BODY()
		
	/** Nominal focus distance of the lens associated with this result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	float EvaluatedFocus = 0.0f;

	/** Nominal focal length of the lens associated with this result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	float EvaluatedZoom = 0.0f;

	/** Final reprojection error produced using this result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	double ReprojectionError = 0.0;

	/** Calibrated focal length result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FFocalLengthInfo FocalLength;

	/** Calibrated image center result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FImageCenterInfo ImageCenter;

	/** Calibrated camera pose for each input image */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	TArray<FTransform> CameraPoses;

	/** Calibrated nodal offset result */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FNodalPointOffset NodalOffset;

	/** Distortion parameters for the model specified by the lens file. And empty parameter array implies that there is a valid ST Map instead. */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FDistortionInfo Parameters;

	/** ST Map that represents the UV displacements for this result. If the ST Map UTexture is not imported by the solver, a path string should be provided so that the lens distortion tool can import it. */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FSTMapInfo STMap;

	/** Absolute path to an ST Map file on disk that should be imported when this result is processed. */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FString STMapFullPath;

	/** Error text to be written by a solver to provide the reason why the solve may have failed. No error message implies that the solve was successful and the result is valid. */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FText ErrorMessage;
};


/** Utility structure for selecting a distortion handler from the camera calibration subsystem */
struct UE_DEPRECATED(5.1, "This struct has been deprecated.") FDistortionHandlerPicker;

USTRUCT(BlueprintType)
struct FDistortionHandlerPicker
{
	GENERATED_BODY()

public:
	/** Default destructor */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FDistortionHandlerPicker() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	/** CineCameraComponent with which the desired distortion handler is associated */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion", Transient)
	TObjectPtr<UCineCameraComponent> TargetCameraComponent = nullptr;

	/** UObject that produces the distortion state for the desired distortion handler */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FGuid DistortionProducerID;

	/** Display name of the desired distortion handler */
	UPROPERTY(BlueprintReadWrite, Category = "Distortion")
	FString HandlerDisplayName;
};
