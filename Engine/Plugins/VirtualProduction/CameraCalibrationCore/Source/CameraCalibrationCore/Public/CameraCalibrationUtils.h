// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CAMERACALIBRATIONCORE_API

/**
 * Helper class for commonly used functions for camera calibration.
 */
class FCameraCalibrationUtils
{
public:
	/** Compares two transforms and returns true if they are nearly equal in distance and angle */
	static UE_API bool IsNearlyEqual(const FTransform& A, const FTransform& B, float MaxLocationDelta = 2.0f, float MaxAngleDegrees = 2.0f);
};

#undef UE_API
