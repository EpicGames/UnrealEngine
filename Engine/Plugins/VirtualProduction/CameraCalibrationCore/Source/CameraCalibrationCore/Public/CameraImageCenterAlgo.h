// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include "CameraImageCenterAlgo.generated.h"

struct FGeometry;
struct FKey;
struct FPointerEvent;

class UImageCenterTool;
class UMaterialInterface;
class SWidget;

/**
 * UCameraImageCenterAlgo defines the interface that any image center algorithm should implement
 * in order to be used and listed by the Image Center Tool.
 */
UCLASS(MinimalAPI, Abstract)
class UCameraImageCenterAlgo : public UObject
{
	GENERATED_BODY()

public:

	/** Make sure you initialize before using the object */
	virtual void Initialize(UImageCenterTool* InImageCenterTool) {};

	/** Clean up resources and don't use ImageCenterTool anymore */
	virtual void Shutdown() {};

	/** Perform any required set when this algo becomes the active one */
	virtual void Activate() {};

	/** Perform any required cleanup when this algo is no longer active */
	virtual void Deactivate() {};

	/** Returns true if this algo is active */
	virtual bool IsActive() const { return bIsActive; }

	/** Called every frame */
	virtual void Tick(float DeltaTime) {};

	/** Callback when viewport is clicked. Returns false if the event was not handled. */
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false; };

	/** Callback when viewport receives input key presses. Returns false if the event was not handled. */
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) { return false; };

	/** Returns the UI of this calibrator. Expected to only be called once */
	virtual TSharedRef<SWidget> BuildUI() { return SNullWidget::NullWidget; };

	/** Returns true if the algo has changed the image center for the current focus/zoom */
	virtual bool HasImageCenterChanged() { return false; };

	/** Returns a descriptive name/title of this image center algorithm */
	virtual FName FriendlyName() const { return TEXT("Invalid Name"); };

	/** Returns the overlay material used by this algo (if any) */
	virtual UMaterialInterface* GetOverlayMaterial() const { return nullptr; };

	/** Returns true is this algo has enabled an overlay */
	virtual bool IsOverlayEnabled() const { return false; };

	/** Called when the current offset was saved */
	virtual void OnSavedImageCenter() { };

	/** Called to present the user with instructions on how to use this algo */
	virtual TSharedRef<SWidget> BuildHelpWidget() { return SNew(STextBlock).Text(FText::FromString(TEXT("Coming soon!"))); };

protected:

	/** Whether this algo is current active or not */
	bool bIsActive = false;
};
