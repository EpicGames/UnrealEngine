// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tasks/Task.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include "CameraLensDistortionAlgo.generated.h"

struct FGeometry;
struct FPointerEvent;
struct FDistortionCalibrationResult;

struct FDistortionInfo;
struct FFocalLengthInfo;
struct FImage;
struct FImageCenterInfo;

class FJsonObject;
class ULensDistortionTool;
class ULensModel;
class UMaterialInterface;
class SWidget;


typedef UE::Tasks::TTask<FDistortionCalibrationResult> FDistortionCalibrationTask;

/**
 * Defines the interface that any lens distortion algorithm should implement
 * in order to be used and listed by the Lens Distortion Tool. 
 */
UCLASS(MinimalAPI, Abstract)
class UCameraLensDistortionAlgo : public UObject
{
	GENERATED_BODY()

public:

	/** Make sure you initialize before using the object */
	virtual void Initialize(ULensDistortionTool* InTool) {};

	/** Clean up resources */
	virtual void Shutdown() {};

	/** Called every frame */
	virtual void Tick(float DeltaTime) {};

	/** Returns true if the algo supports the input Lens Model */
	virtual bool SupportsModel(const TSubclassOf<ULensModel>& LensModel) const { return false; };

	/** Callback when viewport is clicked. Returns false if the event was not handled. */
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false;  };

	/** Returns the UI of this calibrator. Expected to only be called once */
	virtual TSharedRef<SWidget> BuildUI() { return SNullWidget::NullWidget; };

	/** Returns a descriptive name/title for this algorithm */
	virtual FName FriendlyName() const { return TEXT("Invalid Name"); };

	/** Returns a shorter name for this algorithm */
	virtual FName ShortName() const { return TEXT("Invalid"); };

	/** Returns the overlay material used by this algo (if any) */
	virtual UMaterialInterface* GetOverlayMaterial() const { return nullptr; };

	/** Returns true is this algo has enabled an overlay */
	virtual bool IsOverlayEnabled() const { return false; };

	/** Called when the data sample was saved to the lens file */
	virtual void OnDistortionSavedToLens() { };

	/** Returns the lens distortion calibration data */
	virtual bool GetLensDistortion(
		float& OutFocus,
		float& OutZoom,
		FDistortionInfo& OutDistortionInfo,
		FFocalLengthInfo& OutFocalLengthInfo,
		FImageCenterInfo& OutImageCenterInfo,
		TSubclassOf<ULensModel>& OutLensModel, 
		double& OutError, 
		FText& OutErrorMessage) 
	{ 
		return false; 
	};

	/** Launches an asynchronous task to perform the distortion calibration. The task will contain the calibration result when it finishes. */
	virtual FDistortionCalibrationTask BeginCalibration(FText& OutErrorMessage) { return {}; };

	/** Cancel an in-progress async calibration task */
	virtual void CancelCalibration() { };

	/** Get the latest status from the lens distortion calibration. Returns true if this status is "new", or false if this status was previously reported. */
	virtual bool GetCalibrationStatus(FText& StatusText) const { return false; };

	/** 
	 * If true, the algo must override BeginCalibration() and use it to launch the calibration as an asynchronous task.
	 * If false, the algo must override GetLensDistortion() and run the calibration on the Game Thread.
	 */
	virtual bool SupportsAsyncCalibration() { return false; };

	/** Returns true if there is any existing calibration data */
	virtual bool HasCalibrationData() const { return false; };

	/** Performs any necessary steps (such as clearing existing calibration data) before importing a calibration dataset */
	virtual void PreImportCalibrationData() { };

	/** Import a JsonObject of calibration data that is needed by the algorithm, but is not associated with a single row of data */
	virtual void ImportSessionData(const TSharedRef<FJsonObject>& SessionDataObject) { };

	/** 
	 * Import a JsonObject of calibration data that represents a single calibration row. 
	 * RowImage is not guaranteed to have any data. The algorithm should check its validity before using it.
	 * Returns the row index of the imported row.
	 */
	virtual int32 ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage) { return -1; };

	/** Performs any necessary steps after importing a calibration dataset */
	virtual void PostImportCalibrationData() { };

	/** Called to present the user with instructions on how to use this algo */
	virtual TSharedRef<SWidget> BuildHelpWidget() { return SNew(STextBlock).Text(FText::FromString(TEXT("Coming soon!"))); };
};
