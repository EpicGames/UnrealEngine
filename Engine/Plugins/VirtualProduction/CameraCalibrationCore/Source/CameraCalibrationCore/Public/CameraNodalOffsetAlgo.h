// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include "CameraNodalOffsetAlgo.generated.h"

struct FGeometry;
struct FImage;
struct FKey;
struct FNodalPointOffset;
struct FPointerEvent;

class FJsonObject;
class UMaterialInterface;
class UNodalOffsetTool;
class SWidget;

/**
 * UCameraNodalOffsetAlgo defines the interface that any nodal calibration point algorithm should implement
 * in order to be used and listed by the Nodal Offset Tool. 
 */
UCLASS(MinimalAPI, Abstract)
class UCameraNodalOffsetAlgo : public UObject
{
	GENERATED_BODY()

public:

	/** Make sure you initialize before using the object */
	virtual void Initialize(UNodalOffsetTool* InNodalOffsetTool) {};

	/** Clean up resources and don't use NodalOffsetTool anymore */
	virtual void Shutdown() {};

	/** Called every frame */
	virtual void Tick(float DeltaTime) {};

	/** Callback when viewport is clicked. Returns false if the event was not handled. */
	virtual bool OnViewportClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) { return false;  };

	/** Callback when viewport receives input key presses. Returns false if the event was not handled. */
	virtual bool OnViewportInputKey(const FKey& InKey, const EInputEvent& InEvent) { return false; };

	/** Callback when the user performs a marquee select operation. Returns false if the event was not handled. */
	virtual bool OnViewportMarqueeSelect(FVector2D StartPosition, FVector2D EndPosition) { return false; };

	/** Returns the UI of this calibrator. Expected to only be called once */
	virtual TSharedRef<SWidget> BuildUI() { return SNullWidget::NullWidget; };

	/** Returns the most recently calibrated nodal offset transform, with an error metric */
	virtual bool GetNodalOffset(FNodalPointOffset& OutNodalOffset, float& OutFocus, float& OutZoom, float& OutError, FText& OutErrorMessage) { return false; };

	/** Returns a descriptive name/title for this algorithm */
	virtual FName FriendlyName() const { return TEXT("Invalid Name"); };

	/** Returns a shorter name for this algorithm */
	virtual FName ShortName() const { return TEXT("Invalid"); };

	/** Returns the overlay material used by this algo (if any) */
	virtual UMaterialInterface* GetOverlayMaterial() const { return nullptr; };

	/** Returns true is this algo has enabled an overlay */
	virtual bool IsOverlayEnabled() const { return false; };

	/** Called when the current offset was saved */
	virtual void OnSavedNodalOffset() { };

	/** Returns true if there is any existing calibration data */
	virtual bool HasCalibrationData() const { return false; };

	/** Performs any necessary steps (such as clearing existing calibration data) before importing a calibration dataset */
	virtual void PreImportCalibrationData() { };

	/** Import a JsonObject of calibration data that is needed by the algorithm, but is not associated with a single row of data */
	virtual void ImportSessionData(const TSharedRef<FJsonObject>& SessionDataObject) { };

	/**
	 * Import a JsonObject of calibration data that represents a single calibration row.
	 * RowImage is not guaranteed to have any data. The algorithm should check its validity before using it.
	 * Returns the row index of the imported row.
	 */
	virtual int32 ImportCalibrationRow(const TSharedRef<FJsonObject>& CalibrationRowObject, const FImage& RowImage) { return -1; };

	/** Performs any necessary steps after importing a calibration dataset */
	virtual void PostImportCalibrationData() { };

	/** Called to present the user with instructions on how to use this algo */
	virtual TSharedRef<SWidget> BuildHelpWidget() { return SNew(STextBlock).Text(FText::FromString(TEXT("Coming soon!")));  };
};
