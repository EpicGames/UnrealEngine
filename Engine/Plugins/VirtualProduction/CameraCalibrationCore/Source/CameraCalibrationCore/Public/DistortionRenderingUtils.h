// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"

class UTextureRenderTarget2D;

namespace DistortionRenderingUtils
{
	UE_DEPRECATED(5.4, "The version of UndistortImagePoints that takes arrays of FVector2Ds as arguments is deprecated. Please use the version that take FVector2f arguments.")
	CAMERACALIBRATIONCORE_API void UndistortImagePoints(UTextureRenderTarget2D* DistortionMap, TArray<FVector2D> ImagePoints, TArray<FVector2D>& OutUndistortedPoints);

	/** Undistort the input array of 2D image points using the input displacement map */
	CAMERACALIBRATIONCORE_API void UndistortImagePoints(UTextureRenderTarget2D* DistortionMap, TArray<FVector2f> ImagePoints, TArray<FVector2f>& OutUndistortedPoints);
};
