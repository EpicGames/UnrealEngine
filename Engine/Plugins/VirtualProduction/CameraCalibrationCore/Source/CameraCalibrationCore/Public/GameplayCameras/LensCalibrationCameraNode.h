// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Core/CameraNode.h"

#include "LensCalibrationCameraNode.generated.h"

class ULensFile;

/**
 * A camera node that specifies a lens definition.
 */
UCLASS(MinimalAPI, meta=(CameraNodeCategories="Common,Lens"))
class ULensCalibrationCameraNode : public UCameraNode
{
	GENERATED_BODY()

protected:

	// UCameraNode interface.
	virtual FCameraNodeEvaluatorPtr OnBuildEvaluator(FCameraNodeEvaluatorBuilder& Builder) const override;

public:

	/** The lens definition to use. */
	UPROPERTY(EditAnywhere, Category="Lens Calibration")
	TObjectPtr<ULensFile> LensFile;
};

