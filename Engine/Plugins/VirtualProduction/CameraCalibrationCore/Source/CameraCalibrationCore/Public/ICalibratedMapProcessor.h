// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FDerivedDistortionDataJobOutput;
struct FDerivedDistortionDataJobArgs;

/** Delegate triggered when derived data job has completed, successfully or not. */
DECLARE_DELEGATE_OneParam(FOnDerivedDistortionJobCompleted, const FDerivedDistortionDataJobOutput& /*JobOutput*/);


/**
 * Interface to handle stmaps (calibrated maps) and derive distortion data out of it
 */
class ICalibratedMapProcessor
{
public:

	virtual ~ICalibratedMapProcessor() = default;

	/** Updates processor to manage job queues */
	virtual void Update() = 0;

	/** Push new job to derive distortion data from calibrated map */
	virtual bool PushDerivedDistortionDataJob(FDerivedDistortionDataJobArgs&& JobArgs) = 0;
};