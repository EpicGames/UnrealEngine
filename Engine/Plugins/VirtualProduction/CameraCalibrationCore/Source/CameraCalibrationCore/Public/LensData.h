// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "CalibratedMapFormat.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "Models/LensModel.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "LensData.generated.h"


/**
 * Information about the lens rig
 */
USTRUCT(BlueprintType)
struct FLensInfo
{
	GENERATED_BODY()

public:
	bool operator==(const FLensInfo& Other) const
	{
		return LensModelName == Other.LensModelName
			&& LensSerialNumber == Other.LensSerialNumber
			&& LensModel == Other.LensModel
			&& ImageDimensions == Other.ImageDimensions
			&& SensorDimensions == Other.SensorDimensions;
	}
	
public:

	/** Model name of the lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens Info")
	FString LensModelName;

	/** Serial number of the lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens Info")
	FString LensSerialNumber;

	/** Model of the lens (spherical, anamorphic, etc...) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens Info")
	TSubclassOf<ULensModel> LensModel;

	/** Width and height of the calibrated camera's sensor, in millimeters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info")
	FVector2D SensorDimensions = FVector2D(23.76f, 13.365f);

	/** 
	  * Resolution of the original footage that was captured by the camera (not necessarily the resolution of the media source).
	  * For example, the original footage might have been 4320x1746, but to transmit that image over SDI, it might have been scaled and fit into a 4096x2160 frame.
	  * In this case, the "Image Resolution" should be set to 4320x1746, while the "Media Resolution" will read 4096x2160.
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info", meta = (DisplayName = "Image Resolution"))
	FIntPoint ImageDimensions = FIntPoint(1920, 1080);

	/** Squeeze Factor (or Pixel Aspect) for anamorphic lenses. Spherical Lenses should keep this default to 1.0f */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Info")
	float SqueezeFactor = 1.0f;
};

/**
 * Information about a camera feed, including its dimensions and aspect ratio.
 * The camera feed represents (potentially) an inner rect of a media input, excluding any masks / extractions that may surround it.
 */
USTRUCT(BlueprintType)
struct FCameraFeedInfo
{
	GENERATED_BODY()

public:
	FIntPoint GetDimensions() const
	{
		return Dimensions;
	}

	void SetDimensions(FIntPoint InDimensions, bool bMarkAsOverridden = false)
	{
		Dimensions = InDimensions;
		bIsOverridden = bMarkAsOverridden;

		// If one of the dimensions is set to 0, the description of the feed is invalid
		if (Dimensions.X == 0 || Dimensions.Y == 0)
		{
			AspectRatio = 0.0f;
			bIsValid = false;
		}
		else
		{
			AspectRatio = Dimensions.X / (float)Dimensions.Y;
			bIsValid = true;
		}
	}

	float GetAspectRatio() const
	{
		return AspectRatio;
	}

	void Invalidate()
	{
		Dimensions = FIntPoint(0, 0);
		AspectRatio = 0.0f;
		bIsValid = false;
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	bool IsOverridden() const
	{
		return bIsOverridden;
	}

private:
	/** Dimensions of the Camera Feed */
	UPROPERTY(EditAnywhere, Category = "Camera Feed Info")
	FIntPoint Dimensions = FIntPoint(0, 0);

	/** Aspect Ratio of the Camera Feed */
	UPROPERTY(VisibleAnywhere, Category = "Camera Feed Info")
	float AspectRatio = 0.0f;

	/** Whether the dimensions of the feed are valid (false if either X or Y is 0) */
	bool bIsValid = false;

	/** Whether the dimensions of the feed were overridden by the user */
	bool bIsOverridden = false;
};

/**
 * Information about the simulcam composition
 */
USTRUCT(BlueprintType)
struct FSimulcamInfo
{
	GENERATED_BODY()

public:
	/** Resolution of the selected Media Source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Media Info")
	FIntPoint MediaResolution = FIntPoint(0, 0);

	/** Aspect ratio of the media plate in the simulcam composition, which was set to match the aspect ratio of the selected Media Source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Simulcam Info")
	float MediaPlateAspectRatio = 0.0f;

	/** Aspect ratio of the CG layer in the simulcam composition, which was set to match the aspect ratio of the selected Camera Actor */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Simulcam Info", meta = (DisplayName = "CG Layer Aspect Ratio"))
	float CGLayerAspectRatio = 0.0f;
};

/**
 * Lens distortion parameters
 */
USTRUCT(BlueprintType)
struct FDistortionInfo
{
	GENERATED_BODY()

public:
	/** Generic array of floating-point lens distortion parameters */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	TArray<float> Parameters;
};

/**
 * Normalized focal length information for both width and height dimension
 * If focal length is in pixel, normalize using pixel dimensions
 * If focal length is in mm, normalize using sensor dimensions
 */
USTRUCT(BlueprintType)
struct FFocalLengthInfo
{
	GENERATED_BODY()

public:

	/** Value expected to be normalized (unitless) */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Camera")
	FVector2D FxFy = FVector2D(1.0f, (16.0f / 9.0f));
};

/**
 * Pre generate STMap and normalized focal length information
 */
USTRUCT(BlueprintType)
struct FSTMapInfo
{
	GENERATED_BODY()

public:
	/** Pre calibrated UVMap/STMap */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	TObjectPtr<UTexture> DistortionMap = nullptr;

	/** Calibrated map format */
	UPROPERTY(EditAnywhere, Category = "Format", meta = (ShowOnlyInnerProperties))
	FCalibratedMapFormat MapFormat;
};

/**
 * Lens camera image center parameters
 */
USTRUCT(BlueprintType)
struct FImageCenterInfo
{
	GENERATED_BODY()

public:
	/** Value expected to be normalized [0,1] */
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Camera", meta = (DisplayName = "Image Center"))
	FVector2D PrincipalPoint = FVector2D(0.5f, 0.5f);
};

/**
 * Lens nodal point offset
 */
USTRUCT(BlueprintType)
struct FNodalPointOffset
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nodal point")
	FVector LocationOffset = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nodal point")
	FQuat RotationOffset = FQuat::Identity;
};

/**
* Distortion data evaluated for given FZ pair based on lens parameters
*/
USTRUCT(BlueprintType)
struct FDistortionData
{
	GENERATED_BODY()

	public:

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Distortion")
	TArray<FVector2D> DistortedUVs;

	/** Estimated overscan factor based on distortion to have distorted cg covering full size */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float OverscanFactor = 1.0f;
};

/**
 * Base struct for point info wrapper which holds focus and zoom
 * Child classes should hold the point info itself
 */
USTRUCT(BlueprintType)
struct FDataTablePointInfoBase
{
	GENERATED_BODY()

	FDataTablePointInfoBase()
		: Focus(0.f)
		, Zoom(0.f)
	{}

	FDataTablePointInfoBase(const float InFocus, const float InZoom)
		: Focus(InFocus)
		, Zoom(InZoom)
	{}

	/** Point Focus Value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	float Focus;

	/** Point Zoom Value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	float Zoom;
};


/**
 * Distortion Point Info struct
 */
USTRUCT(BlueprintType)
struct FDistortionPointInfo : public FDataTablePointInfoBase
{
	GENERATED_BODY()

	using TypeInfo = FDistortionInfo;

	FDistortionPointInfo()
		: FDataTablePointInfoBase()
	{}

	FDistortionPointInfo(const float InFocus, const float InZoom, const FDistortionInfo& InDistortionInfo)
		: FDataTablePointInfoBase(InFocus, InZoom)
		, DistortionInfo(InDistortionInfo)
	{}

	/** Lens distortion parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	FDistortionInfo DistortionInfo;
};


/**
 * Focal Length Point Info struct
 */
USTRUCT(BlueprintType)
struct FFocalLengthPointInfo : public FDataTablePointInfoBase
{
	GENERATED_BODY()

	using TypeInfo = FFocalLengthInfo;

	FFocalLengthPointInfo()
		: FDataTablePointInfoBase()
	{}

	FFocalLengthPointInfo(const float InFocus, const float InZoom, const TypeInfo& InFocalLengthInfo)
		: FDataTablePointInfoBase(InFocus, InZoom)
		, FocalLengthInfo(InFocalLengthInfo)
	{}

	/** Focal Length parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	FFocalLengthInfo FocalLengthInfo;
};

/**
 * ST Map Point Info struct
 */
USTRUCT(BlueprintType)
struct FSTMapPointInfo : public FDataTablePointInfoBase
{
	GENERATED_BODY()

	using TypeInfo = FSTMapInfo;

	FSTMapPointInfo()
		: FDataTablePointInfoBase()
	{}

	FSTMapPointInfo(const float InFocus, const float InZoom, const TypeInfo& InSTMapInfo)
		: FDataTablePointInfoBase(InFocus, InZoom)
		, STMapInfo(InSTMapInfo)
	{}

	/** ST Map parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	FSTMapInfo STMapInfo;
};

/**
 * Image Center Point Info struct
 */
USTRUCT(BlueprintType)
struct FImageCenterPointInfo : public FDataTablePointInfoBase
{
	GENERATED_BODY()

	using TypeInfo = FImageCenterInfo;

	FImageCenterPointInfo()
		: FDataTablePointInfoBase()
	{}

	FImageCenterPointInfo(const float InFocus, const float InZoom, const TypeInfo& InImageCenterInfo)
		: FDataTablePointInfoBase(InFocus, InZoom)
		, ImageCenterInfo(InImageCenterInfo)
	{}

	/** Image Center parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	FImageCenterInfo ImageCenterInfo;
};

/**
 * Nodal Point Point Info struct
 */
USTRUCT(BlueprintType)
struct FNodalOffsetPointInfo : public FDataTablePointInfoBase
{
	GENERATED_BODY()

	using TypeInfo = FNodalPointOffset;

	FNodalOffsetPointInfo()
		: FDataTablePointInfoBase()
	{}

	FNodalOffsetPointInfo(const float InFocus, const float InZoom, const TypeInfo& InNodalPointOffset)
		: FDataTablePointInfoBase(InFocus, InZoom)
		, NodalPointOffset(InNodalPointOffset)
	{}

	/** Nodal Point parameter */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Point")
	FNodalPointOffset NodalPointOffset;
};


