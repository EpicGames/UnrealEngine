// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CameraCalibrationTypes.h"
#include "CineCameraSettings.h"
#include "CoreMinimal.h"
#include "LensData.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/Guid.h"
#include "Models/LensModel.h"

#include "LensDistortionModelHandlerBase.generated.h"

// Forward declarations for RDG types
class FRDGBuilder;
class FRDGTexture;
using FRDGTextureRef = FRDGTexture*;

#define UE_API CAMERACALIBRATIONCORE_API

USTRUCT(BlueprintType)
struct FLensDistortionState
{
	GENERATED_BODY()

public:
	/** Generic array of distortion parameters */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	FDistortionInfo DistortionInfo;

	/** Normalized focal fength in both dimensions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	FFocalLengthInfo FocalLengthInfo;
	
	/** Normalized center of the image, in the range [0.0f, 1.0f] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion", meta = (DisplayName = "Image Center"))
	FImageCenterInfo ImageCenter;

public:
	UE_API bool operator==(const FLensDistortionState& Other) const;
	bool operator!=(const FLensDistortionState& Other) const { return !(*this == Other); }
};

/** Asset user data that can be used on Camera Actors to manage lens distortion state and utilities  */
UCLASS(MinimalAPI, Abstract)
class ULensDistortionModelHandlerBase : public UObject
{
	GENERATED_BODY()

public:
	UE_API ULensDistortionModelHandlerBase();

	/** Returns true if the input model is supported by this model handler, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = "Distortion")
	UE_API bool IsModelSupported(const TSubclassOf<ULensModel>& ModelToSupport) const;

	/** Update the lens distortion state, recompute the overscan factor, and set all material parameters */
	UFUNCTION(BlueprintCallable, Category = "Distortion")
    UE_API void SetDistortionState(const FLensDistortionState& InNewState);

	/** Get the UV displacement map used to undistort a distorted image */
	UFUNCTION(BlueprintCallable, Category = "Distortion")
	UTextureRenderTarget2D* GetUndistortionDisplacementMap() const { return UndistortionDisplacementMapRT; }

	/** Get the UV displacement map used to distort an undistorted image */
	UFUNCTION(BlueprintCallable, Category = "Distortion")
	UTextureRenderTarget2D* GetDistortionDisplacementMap() const { return DistortionDisplacementMapRT; }

public:
	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif	
	//~ End UObject Interface

	/** Get the current distortion state (the lens model and properties that mathematically represent the distortion characteristics */
	FLensDistortionState GetCurrentDistortionState() const { return CurrentState; }

	/** Specify the filmback settings of the CineCamera that is being used for distortion */
	UE_API void SetCameraFilmback(const FCameraFilmbackSettings& InCameraFilmback);

	/** Get the post-process MID for the currently specified lens model */
	UMaterialInstanceDynamic* GetDistortionMID() const { return DistortionPostProcessMID; }

	/** Get the specified lens model that characterizes the distortion effect */
	const TSubclassOf<ULensModel>& GetLensModelClass() const { return LensModelClass; };

	/** Get the UObject that produces the distortion state for this handler */
	FGuid GetDistortionProducerID() const { return DistortionProducerID; }

	/** Set the UObject that produces the distortion state for this handler */
	void SetDistortionProducerID(const FGuid& InDistortionProducerID) { DistortionProducerID = InDistortionProducerID; }

	/** Get the display name of this lens distortion model handler */
	FString GetDisplayName() const { return DisplayName; }

	/** Set the display name of this lens distortion model handler */
	void SetDisplayName(FString InDisplayName) { DisplayName = InDisplayName; }

	/** Get the normalized center of projection of the image, in the range [0.0f, 1.0f] */
	FVector2D GetPrincipalPoint() const { return CurrentState.ImageCenter.PrincipalPoint; }

	/** Get the normalized focal length (unitless) */
	FVector2D GetFxFy() const { return CurrentState.FocalLengthInfo.FxFy; }

	/** Updates overscan factor and applies to material instances */
	UE_API void SetOverscanFactor(float OverscanFactor);

	/** Returns the last overscan factor that was set */
	float GetOverscanFactor() const { return OverscanFactor; }

	/** Use the current distortion state to compute the overscan factor needed such that all distorted UVs will fall into the valid range of [0,1] */
	UE_API float ComputeOverscanFactor() const;

	/** Use the current distortion state to compute the overscan factor needed when undistorting a distorted image */
	UE_API float ComputeInverseOverscanFactor() const;
	
	/** Computes the distorted version of UndistortedUVs based on the current state */
	UE_API TArray<FVector2D> GetDistortedUVs(TConstArrayView<FVector2D> UndistortedUVs) const;

	/** Draw the undistortion displacement map associated with the current state to the DestinationTexture */
	UE_API bool DrawUndistortionDisplacementMap(UTextureRenderTarget2D* DestinationTexture);

	/** Draw the distortion displacement map associated with the current state to the DestinationTexture */
	UE_API bool DrawDistortionDisplacementMap(UTextureRenderTarget2D* DestinationTexture);

	/** Draws the current distortion state to the internal displacement map */
	UE_API void ProcessCurrentDistortion();

	/** Create two displacement maps, for distortion and undistortion, using the input resolution */
	UE_API void CreateDisplacementMaps(const FIntPoint DisplacementMapResolution);

	/** Execute the distortion shader with the specified parameters, which enables a modular SVE that doesn't need to know lens model specifics. */
	virtual void ExecuteDistortionShader(class FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const {}

	/** Returns true if this model performs forward distortion (needs inverse overscan), false if reverse distortion (needs forward overscan) */
	virtual bool IsForwardDistorting() const { return true; }

	/** 
	 * Returns the preferred rendering mode for this lens model.
	 * By default, reverse distorting models prefer PPM for accuracy, forward distorting models prefer SVE.
	 * Override this to customize the preference for specific lens models.
	 * Return Preferred to use the default behavior based on IsForwardDistorting().
	 */
	UE_API virtual EDistortionRenderingMode GetPreferredRenderingMode() const;

protected:
	/** Initialize the handler. Derived classes must set the LensModelClass that they support, if not already set */
	virtual void InitializeHandler() PURE_VIRTUAL(ULensDistortionModelHandlerBase::InitializeHandler);

	/** Use the current distortion state to compute the distortion position of an input UV coordinate */
	virtual FVector2D ComputeDistortedUV(const FVector2D& InScreenUV) const PURE_VIRTUAL(ULensDistortionModelHandlerBase::ComputeDistortedUV, return FVector2D::ZeroVector;);

	/** Use the current distortion state to compute the undistortion position of an input UV coordinate */
	virtual FVector2D ComputeUndistortedUV(const FVector2D& InScreenUV) const PURE_VIRTUAL(ULensDistortionModelHandlerBase::ComputeUndistortedUV, return FVector2D::ZeroVector;);
	
	/** Create the distortion MIDs */
	virtual void InitDistortionMaterials() PURE_VIRTUAL(ULensDistortionModelHandlerBase::InitDistortionMaterials);

	/** Set the material parameters for the displacement map and distortion post-process materials */
	virtual void UpdateMaterialParameters() PURE_VIRTUAL(ULensDistortionModelHandlerBase::UpdateMaterialParameters);

	/** Convert the generic distortion parameter array into the specific structure of parameters used by the supported lens model */
	virtual void InterpretDistortionParameters() PURE_VIRTUAL(ULensDistortionModelHandlerBase::InterpretDistortionParameters);

	/** Return the shader path for this model's distortion compute shader, promoting a modular shader architecture */
	virtual FString GetDistortionShaderPath() const { return FString(); }

private:
	/** Tests whether the input material's shaders have been compiled, indicating the material can be used for drawing */
	UE_API bool IsDisplacementMapMaterialReady(UMaterialInstanceDynamic* MID);

protected:
	/** Lens Model describing how to interpret the distortion parameters */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Distortion")
	TSubclassOf<ULensModel> LensModelClass;

	/** Dynamically created post-process material instance for the currently specified lens model */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Distortion")
	TObjectPtr<UMaterialInstanceDynamic> DistortionPostProcessMID = nullptr;

	/** Current state as set by the most recent call to Update() */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Distortion", meta = (ShowOnlyInnerProperties))
	FLensDistortionState CurrentState;

	/** Filmback settings of the camera that is being used for distortion */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	FCameraFilmbackSettings CameraFilmback;

	/** Display name, used to identify handler in-editor details panels */
	UPROPERTY(VisibleAnywhere, Category = "Distortion")
	FString DisplayName;

	/** Computed overscan factor needed to scale the camera's FOV (read-only) */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Distortion")
	float OverscanFactor = 1.0f;

	/** MID used to draw the undistortion displacement map */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> UndistortionDisplacementMapMID = nullptr;

	/** MID used to draw the distortion displacement map */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> DistortionDisplacementMapMID = nullptr;

	/** UV displacement map used to undistort a distorted image */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> UndistortionDisplacementMapRT = nullptr;

	/** UV displacement map used to distort an undistorted image */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> DistortionDisplacementMapRT = nullptr;

	/** UObject that is producing the distortion state for this handler */
	UPROPERTY()
	FGuid DistortionProducerID;

private:
	/** Tracks whether distortion state has been changed */
	bool bIsDirty = true;
};

#undef UE_API
