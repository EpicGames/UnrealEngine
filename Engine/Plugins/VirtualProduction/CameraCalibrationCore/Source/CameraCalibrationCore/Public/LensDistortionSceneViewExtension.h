// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

#include "LensFileRendering.h"

struct FLensDistortionLUT;
class UCameraComponent;

/**
 * View extension drawing distortion/undistortion displacement maps
 */
class FLensDistortionSceneViewExtension : public FSceneViewExtensionBase
{
public:

	FLensDistortionSceneViewExtension(const FAutoRegister& AutoRegister);

	//~ Begin ISceneViewExtension interface	
	virtual int32 GetPriority() const override;
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;
	//~End ISceneVIewExtension interface

	/** Render forward & inverse displacement maps and return the engine lens distortion LUT. */
	CAMERACALIBRATIONCORE_API bool RenderViewDistortionLUT(FRDGBuilder& GraphBuilder, uint32 InViewActorID, FLensDistortionLUT& ViewDistortionLUT) const;

public:
	/** Update the distortion state and blending params for the input camera */
	void UpdateDistortionState_AnyThread(ACameraActor* CameraActor, FDisplacementMapBlendingParams DistortionState, ULensDistortionModelHandlerBase* LensDistortionHandler, EDistortionRenderingMode InRenderingMode);

	/** Returns true if any distortion state has been registered. */
	CAMERACALIBRATIONCORE_API bool HasDistortionState_AnyThread() const;

	/** Returns the registered distortion rendering mode for the given view actor, or unset if no proxy exists. */
	CAMERACALIBRATIONCORE_API TOptional<EDistortionRenderingMode> GetRenderingMode_AnyThread(uint32 InViewActorID) const;

	/** Remove the distortion state and blending params for the input camera */
	void ClearDistortionState_AnyThread(ACameraActor* CameraActor);

private:
	/** Use the input distortion state to draw a distortion displacement map */
	void DrawDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FLensDistortionState& CurrentState, class ULensDistortionModelHandlerBase* ModelHandler, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMapWithOverscan) const;

	/** Use the input blend parameters to draw multiple displacement maps and blend them together into a final distortion displacement map */
	void BlendDisplacementMaps_RenderThread(FRDGBuilder& GraphBuilder, const FDisplacementMapBlendingParams& BlendState, class ULensDistortionModelHandlerBase* ModelHandler, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMapWithOverscan) const;

	/** Crop the input overscanned distortion map to the original requested resolution */
	void CropDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDistortionMapWithOverscan, FRDGTextureRef& OutDistortionMap) const;

	/** Invert the input distortion map to generate a matching undistortion map (with no overscan) */
	void InvertDistortionMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDistortionMap, float InInverseOverscan, FRDGTextureRef& OutUndistortionMap) const;

	/** Performs a warp grid inverse of the specified distortion map to fill up its inverse to the specified overscan amount */
	void FillSTDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InUndisplacementMap, float InOverscan, FRDGTextureRef& OutFilledDisplacementMap) const;

	/** Recenters the specified distortion map within an overscanned map, filling the overscanned region with zeros */
	void RecenterSTDisplacementMap_RenderThread(FRDGBuilder& GraphBuilder, const FRDGTextureRef& InDisplacementMap, float InOverscan, FRDGTextureRef& OutRecenteredDisplacementMap) const;
	
	/** Post-process pass where we optionally apply the distortion material. */
	FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessMaterialInputs& Inputs, UMaterialInterface* LensDistortionMaterial);

private:

	struct FCameraDistortionProxy
	{
		FDisplacementMapBlendingParams Params = {};
		float CameraOverscan = 1.0f;
		FCameraFilmbackSettings FilmbackSettings = {};
		TWeakObjectPtr<ULensDistortionModelHandlerBase> LensDistortionHandler = nullptr;
		EDistortionRenderingMode RenderingMode = EDistortionRenderingMode::Preferred;
	};
	/** Map of cameras to their associated distortion state and blending parameters, used to determine if and how displacement maps should be rendered for a specific view */
	TMap<uint32, FCameraDistortionProxy> DistortionStateMap;

	/** Critical section to lock access to the distortion state map when potentially being accessed from multiple threads */
	mutable FCriticalSection DistortionStateMapCriticalSection;
};
