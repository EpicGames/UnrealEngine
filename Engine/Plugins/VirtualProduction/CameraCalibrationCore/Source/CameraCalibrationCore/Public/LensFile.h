// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "UObject/Object.h"

#include "CineCameraComponent.h"
#include "CoreTypes.h"
#include "Engine/Texture.h"
#include "ICalibratedMapProcessor.h"
#include "LensData.h"
#include "Misc/Optional.h"
#include "Tables/DistortionParametersTable.h"
#include "Tables/EncodersTable.h"
#include "Tables/FocalLengthTable.h"
#include "Tables/ImageCenterTable.h"
#include "Tables/NodalOffsetTable.h"
#include "Tables/STMapTable.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectMacros.h"

#include "LensFile.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API


class UCineCameraComponent;
class ULensDistortionModelHandlerBase;
class UTextureRenderTarget2D;
struct FBaseLensTable;
struct FDisplacementMapBlendingParams;

#if WITH_EDITORONLY_DATA
class UAssetImportData;
#endif

DECLARE_MULTICAST_DELEGATE_OneParam(FOnLensFileModelChanged, const TSubclassOf<ULensModel>&);


/** Mode of operation of Lens File */
UENUM()
enum class ELensDataMode : uint8
{
	Parameters = 0,
	STMap = 1
};

/** Data categories manipulated in the camera calibration tools */
UENUM(BlueprintType)
enum class ELensDataCategory : uint8
{
	Focus,
	Iris,
	Zoom,
	Distortion,
	ImageCenter,
	STMap,
	NodalOffset,
};


/**
 * A Lens file containing calibration mapping from FIZ data
 */
UCLASS(MinimalAPI, BlueprintType)
class ULensFile : public UObject, public FTickableGameObject
{
	GENERATED_BODY()


public:

	UE_API ULensFile();

	//~Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	
#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedEvent ) override;
#endif //WITH_EDITOR
	
	UE_API virtual void PostInitProperties() override;
	//~End UObject interface

	//~ Begin FTickableGameObject
	virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject
	
	/** Returns interpolated distortion parameters */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API bool EvaluateDistortionParameters(float InFocus, float InZoom, FDistortionInfo& OutEvaluatedValue) const;

	/** Returns interpolated focal length */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API bool EvaluateFocalLength(float InFocus, float InZoom, FFocalLengthInfo& OutEvaluatedValue) const;

	/** Returns interpolated image center parameters based on input focus and zoom */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API bool EvaluateImageCenterParameters(float InFocus, float InZoom, FImageCenterInfo& OutEvaluatedValue) const;

	/** Draws the distortion map based on evaluation point*/
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API bool EvaluateDistortionData(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* InLensHandler) const;

	/** Returns interpolated nodal point offset based on input focus and zoom */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API bool EvaluateNodalPointOffset(float InFocus, float InZoom, FNodalPointOffset& OutEvaluatedValue) const;

	/** Whether focus encoder mapping is configured */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool HasFocusEncoderMapping() const;

	/** Returns interpolated focus based on input normalized value and mapping */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API float EvaluateNormalizedFocus(float InNormalizedValue) const;

	/** Whether iris encoder mapping is configured */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool HasIrisEncoderMapping() const;

	/** Returns interpolated iris based on input normalized value and mapping */
	UFUNCTION(BlueprintPure, Category = "Utility")
	UE_API float EvaluateNormalizedIris(float InNormalizedValue) const;

	/** Callbacked when stmap derived data has completed */
	UE_API void OnDistortionDerivedDataJobCompleted(const FDerivedDistortionDataJobOutput& JobOutput);

	/** Update the resolution used for intermediate blending displacement maps and for STMap derived data */
	UE_API void UpdateDisplacementMapResolution(const FIntPoint NewDisplacementMapResolution);

	/** Update the input tolerance used when adding points to calibration tables */
	UE_API void UpdateInputTolerance(const float NewTolerance);

	/** Gets all Distortion points struct with focus, zoom and info  */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API TArray<FDistortionPointInfo> GetDistortionPoints() const;

	/** Gets all Focal Length points struct with focus, zoom and info  */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API TArray<FFocalLengthPointInfo> GetFocalLengthPoints() const;

	/** Gets all ST Map points struct with focus, zoom and info  */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API TArray<FSTMapPointInfo> GetSTMapPoints() const;

	/** Gets all Image Center points struct with focus, zoom and info  */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API TArray<FImageCenterPointInfo> GetImageCenterPoints() const;

	/** Gets all Nodal Offset points struct with focus, zoom and info  */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API TArray<FNodalOffsetPointInfo> GetNodalOffsetPoints() const;

	/** Gets a Distortion point by given focus and zoom, if point does not exists returns false */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool GetDistortionPoint(float InFocus, float InZoom, FDistortionInfo& OutDistortionInfo) const;

	/** Gets a Focal Length point by given focus and zoom, if point does not exists returns false */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool GetFocalLengthPoint(float InFocus, float InZoom, FFocalLengthInfo& OutFocalLengthInfo) const;

	/** Gets a Image Center point by given focus and zoom, if point does not exists returns false */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool GetImageCenterPoint(float InFocus, float InZoom, FImageCenterInfo& OutImageCenterInfo) const;

	/** Gets a Nodal Offset point by given focus and zoom, if point does not exists returns false */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool GetNodalOffsetPoint(float InFocus, float InZoom, FNodalPointOffset& OutNodalPointOffset) const;

	/** Gets a ST Map point by given focus and zoom, if point does not exists returns false */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool GetSTMapPoint(float InFocus, float InZoom, FSTMapInfo& OutSTMapInfo) const;

	/** Adds a distortion point in our map. If a point already exist at the location, it is updated */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void AddDistortionPoint(float NewFocus, float NewZoom, const FDistortionInfo& NewPoint, const FFocalLengthInfo& NewFocalLength);

	/** Adds a focal length point in our map. If a point already exist at the location, it is updated */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void AddFocalLengthPoint(float NewFocus, float NewZoom, const FFocalLengthInfo& NewFocalLength);

	/** Adds an ImageCenter point in our map. If a point already exist at the location, it is updated */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void AddImageCenterPoint(float NewFocus, float NewZoom, const FImageCenterInfo& NewPoint);

	/** Adds an NodalOffset point in our map. If a point already exist at the location, it is updated */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void AddNodalOffsetPoint(float NewFocus, float NewZoom, const FNodalPointOffset& NewPoint);

	/** Adds an STMap point in our map. If a point already exist at the location, it is updated */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void AddSTMapPoint(float NewFocus, float NewZoom, const FSTMapInfo& NewPoint);

	/** Removes a focus point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void RemoveFocusPoint(ELensDataCategory InDataCategory, float InFocus);

	/** Checks to see if there is a focal point for the specified focus in the data category */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API bool HasFocusPoint(ELensDataCategory InDataCategory, float InFocus) const;

	/** Changes the value of a focus point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void ChangeFocusPoint(ELensDataCategory InDataCategory, float InExistingFocus, float InNewFocus);

	/** Merges the contents of one focus point into another focus point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void MergeFocusPoint(ELensDataCategory InDataCategory, float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints);

	/** Removes a zoom point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void RemoveZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InZoom);

	/** Removes a zoom point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API bool HasZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InZoom);

	/** Changes the value of a zoom point */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void ChangeZoomPoint(ELensDataCategory InDataCategory, float InFocus, float InExistingZoom, float InNewZoom);
	
	/** Removes all points of all tables */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void ClearAll();

	/** Removes table associated to data category */
	UFUNCTION(BlueprintCallable, Category = "Lens Table")
	UE_API void ClearData(ELensDataCategory InDataCategory);

	/** Returns whether a category has data samples */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API bool HasSamples(ELensDataCategory InDataCategory) const;

	/** Returns total number of the points for given category */
	UFUNCTION(BlueprintPure, Category = "Lens Table")
	UE_API int32 GetTotalPointNum(ELensDataCategory InDataCategory) const;

	/** Get data table reference based on given category */
	UE_API const FBaseLensTable* GetDataTable(ELensDataCategory InDataCategory) const;

	/** Get data table reference based on given category */
	UE_API FBaseLensTable* GetDataTable(ELensDataCategory InDataCategory);
	
	/** Returns the delegate that is triggered when the LensModel changes */
	FOnLensFileModelChanged& OnLensFileModelChanged() { return OnLensFileModelChangedDelegate; }

	/** Returns the distortion state and blend paramters for input focus and zoom */
	UE_API void GetBlendState(float InFocus, float InZoom, FVector2D InFilmback, FDisplacementMapBlendingParams& OutBlendState);

protected:
	/** Updates derived data entries to make sure it matches what is assigned in map points based on data mode */
	UE_API void UpdateDerivedData();

	/** Create the intermediate displacement maps needed to do map blending to get final distortion/undistortion maps */
	UE_API void CreateIntermediateDisplacementMaps(const FIntPoint DisplacementMapResolution);

	/** Returns the overscan factor based on distorted UV and image center */
	UE_API float ComputeOverscan(const FDistortionData& DerivedData, FVector2D PrincipalPoint) const;

	/** Clears output displacement map on LensHandler to have no distortion and setup distortion data to match that */
	UE_API void SetupNoDistortionOutput(ULensDistortionModelHandlerBase* LensHandler) const;

	/** Evaluates distortion based on InFocus and InZoom using parameters */
	UE_API bool EvaluateDistortionForParameters(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* LensHandler) const;
	
	/** Evaluates distortion based on InFocus and InZoom using STMaps */
	UE_API bool EvaluateDistortionForSTMaps(float InFocus, float InZoom, FVector2D InFilmback, ULensDistortionModelHandlerBase* LensHandler) const;

#if WITH_EDITOR
	/** Builds the lens table focus curves to match the existing data in the tables */
	UE_API void BuildLensTableFocusCurves();
#endif // WITH_EDITOR
	
public:

	/** Lens information */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens info")
	FLensInfo LensInfo;

#if WITH_EDITORONLY_DATA
	/** Camera feed information */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Camera Feed")
	FCameraFeedInfo CameraFeedInfo;

	/** Simulcam information */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Simulcam info")
	FSimulcamInfo SimulcamInfo;
#endif // WITH_EDITORONLY_DATA

	/** Type of data used for lens mapping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens mapping")
	ELensDataMode DataMode = ELensDataMode::Parameters;

	/** Metadata user could enter for its lens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata")
	TMap<FString, FString> UserMetadata;

	/** Encoder mapping table */
	UPROPERTY()
	FEncodersTable EncodersTable;

	/** Distortion parameters table mapping to input focus/zoom  */
	UPROPERTY()
	FDistortionTable DistortionTable;

	/** Focal length table mapping to input focus/zoom  */
	UPROPERTY()
	FFocalLengthTable FocalLengthTable;

	/** Image center table mapping to input focus/zoom  */
	UPROPERTY()
	FImageCenterTable ImageCenterTable;

	/** Nodal offset table mapping to input focus/zoom  */
	UPROPERTY()
	FNodalOffsetTable NodalOffsetTable;

	/** STMap table mapping to input focus/zoom  */
	UPROPERTY()
	FSTMapTable STMapTable;
	
	/** Tolerance used to consider input focus or zoom to be identical */
	float InputTolerance = KINDA_SMALL_NUMBER;

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for importing ulens files. */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "ImportSettings")
	TObjectPtr<UAssetImportData> AssetImportData;
#endif // WITH_EDITORONLY_DATA

protected:

	/** Derived data compute jobs we are waiting on */
	int32 DerivedDataInFlightCount = 0;

	/** Processor handling derived data out of calibrated st maps */
	TUniquePtr<ICalibratedMapProcessor> CalibratedMapProcessor;

	/** Texture used to store temporary undistortion displacement map when using map blending */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UTextureRenderTarget2D>> UndistortionDisplacementMapHolders;

	/** Texture used to store temporary distortion displacement map when using map blending */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UTextureRenderTarget2D>> DistortionDisplacementMapHolders;

	/** The number of intermediate displacement maps needed to do map blending */
	static constexpr int32 DisplacementMapHolderCount = 4;

	/** UV coordinates of 8 points (4 corners + 4 mid points) */
	static UE_API const TArray<FVector2D> UndistortedUVs;

	/** Delegate that is triggered when the LensModel changes */
	FOnLensFileModelChanged OnLensFileModelChangedDelegate;
};


/**
 * Wrapper to facilitate default LensFile vs picker
 */
USTRUCT(BlueprintType)
struct FLensFilePicker
{
	GENERATED_BODY()

public:

	/** Get the proper lens whether it's the default one or the picked one */
	UE_API ULensFile* GetLensFile() const;

public:
	/** If true, default LensFile will be used, if one is set */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens File")
	bool bUseDefaultLensFile = false;

	/** LensFile asset to use if DefaultLensFile is not desired */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Lens File", Meta = (EditCondition = "!bUseDefaultLensFile"))
	TObjectPtr<ULensFile> LensFile = nullptr;
};

USTRUCT(BlueprintType)
struct FLensFileEvaluationInputs
{
	GENERATED_BODY()

public:
	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens File")
	float Focus = 0.0f;

	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens File")
	float Iris = 0.0f;

	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens File")
	float Zoom = 0.0f;

	UPROPERTY(Interp, EditAnywhere, BlueprintReadWrite, Category = "Lens File")
	FCameraFilmbackSettings Filmback;

	UPROPERTY(Transient, BlueprintReadOnly, Category = "Lens File")
	bool bIsValid = false;
};

/** Structure that caches the inputs (and other useful bits) used when evaluating the Lens File */
struct FLensFileEvalData
{
	FLensFileEvalData()
	{
		Invalidate();
	};

	ULensFile* LensFile;

	/** The values that should be used as inputs to the Lut in the LensFile */
	struct
	{
		/** Focus input */
		float Focus;

		/** Iris input */
		float Iris;

		/** Zoom input */
		float Zoom;
	} Input;

	/** Information about the camera associated with the lens evaluation */
	struct
	{
		uint32 UniqueId;
	} Camera;

	/** Information about the Distortion evaluation */
	struct
	{
		/** True if distortion was applied (and the lens distortion handler updated its state) */
		bool bWasEvaluated;

		/** The filmback used when evaluating the distortion data */
		FVector2D Filmback;
	} Distortion;

	/** Information about the nodal offset evaluation */
	struct
	{
		/** True if the evaluated nodal offset was applied to the camera */
		bool bWasApplied;
	} NodalOffset;

	/** Invalidates the data in this structure and avoid using stale or invalid values */
	void Invalidate()
	{
		LensFile = nullptr;

		Input.Focus = 0.0f;
		Input.Iris = 0.0f;
		Input.Zoom = 0.0f;

		Distortion.Filmback = FVector2D(0);

		Camera.UniqueId = INDEX_NONE;
		Distortion.bWasEvaluated = false;
		NodalOffset.bWasApplied = false;
	}
};

#undef UE_API
