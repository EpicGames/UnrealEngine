// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API CAMERACALIBRATIONCORE_API

struct FLensFileObjectVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		EditableFocusCurves,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
	
	UE_API const static FGuid GUID;
	
	FLensFileObjectVersion() = delete;
};

#undef UE_API
