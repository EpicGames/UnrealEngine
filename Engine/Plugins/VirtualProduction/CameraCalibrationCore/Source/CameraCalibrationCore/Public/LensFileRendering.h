// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"

#include "LensDistortionModelHandlerBase.h"

class UTextureRenderTarget2D;

/** Types of blending used which drives permutation */
enum class EDisplacementMapBlendType :uint8
{
	OneFocusOneZoom, //no blending
	OneFocusTwoZoom, //Bezier interp between two zoom points
	TwoFocusOneZoom, //Linear interp between two focus points
	TwoFocusTwoZoom, //Two Besier interp between each pair of zoom points and one linear interp across focus
};

/** Corner of the blending patch which stores x and y values and x and y tangents */
struct FDisplacementMapBlendPatchCorner
{
	FDisplacementMapBlendPatchCorner(float InX, float InY, float InTangentX, float InTangentY)
		: X(InX)
		, Y(InY)
		, TangentX(InTangentX)
		, TangentY(InTangentY)
	{ }

	FDisplacementMapBlendPatchCorner()
		: FDisplacementMapBlendPatchCorner(0.0f, 0.0f, 0.0f, 0.0f)
	{ }
	
	/** Converts the patch corner values to a vector */
	FVector4f ToVector() const { return FVector4f(X, Y, TangentX, TangentY); }
	
	/** X coordinate of the corner */
	float X;

	/** Y coordinate of the corner */
	float Y;

	/** Tangent in the x direction of the corner */
	float TangentX;

	/** Tangent in the y direction of the corner */
	float TangentY;
};

/** Single struct containing blending params for all types */
struct FDisplacementMapBlendingParams
{
	/** Active type of blending */
	EDisplacementMapBlendType BlendType = EDisplacementMapBlendType::OneFocusOneZoom;

	/** Bezier blend parameters */
	float EvalFocus = 0.0f;
	float EvalZoom = 0.0f;

	/** Corners of the blending patch, indexed in the following order: (X0, Y0) -> (X1, Y0) -> (X1, Y1) -> (X0, Y1) */
	FDisplacementMapBlendPatchCorner PatchCorners[4];
	
	/** Distortion state for each of four possible corners to be blended */
	FLensDistortionState States[4];

	/** Scale parameter that allows displacement maps for one sensor size to be applied to camera's with a different sensor size */
	FVector2D FxFyScale = { 1.0f, 1.0f };

	/** Image center parameter to compute center shift needed to offset resulting map */
	FVector2D PrincipalPoint = { 0.5f, 0.5f };
};

namespace LensFileRendering
{
	/** Clears the given render target. Useful when no distortion can be applied and the RT has to be resetted */
	void ClearDisplacementMap(UTextureRenderTarget2D* OutRenderTarget);

	/**
	 * Draws the blended result of displacement map from input textures based on blend parameters
	 * One texture is always needed to do a passthrough. Up to four textures can be blended using bilinear
	 */
	bool DrawBlendedDisplacementMap(UTextureRenderTarget2D* OutRenderTarget
		, const FDisplacementMapBlendingParams& BlendParams
		, UTextureRenderTarget2D* SourceTextureOne
		, UTextureRenderTarget2D* SourceTextureTwo = nullptr
		, UTextureRenderTarget2D* SourceTextureThree = nullptr
		, UTextureRenderTarget2D* SourceTextureFour = nullptr);
}
