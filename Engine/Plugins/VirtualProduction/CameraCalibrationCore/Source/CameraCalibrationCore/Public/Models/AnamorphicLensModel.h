// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensModel.h"

#include "AnamorphicLensModel.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/**
 * Lens distortion parameters for the 3DE4 Anamorphic - Standard Degree 4 model
 * All parameters are unitless and represent the coefficients used to undistort a distorted image
 * For complete model description, see "tde4_ldm_standard.pdf" from https://www.3dequalizer.com/ in the Lens Distortion Plugin Kit v2.8 
 */
USTRUCT(BlueprintType)
struct FAnamorphicDistortionParameters
{
	GENERATED_BODY()

public:
	/** Anamorphic Squeeze (the ratio of the filmback size to the size of the rasterized image) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float PixelAspect = 1.0f;

	/** X coefficient of the r^2 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CX02 = 0.0f;

	/** X coefficient of the r^4 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CX04 = 0.0f;

	/** X coefficient of the r^2*cos(2*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CX22 = 0.0f;

	/** X coefficient of the r^4*cos(2*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CX24 = 0.0f;

	/** X coefficient of the r^4*cos(4*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CX44 = 0.0f;

	/** Y coefficient of the r^2 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CY02 = 0.0f;

	/** Y coefficient of the r^4 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CY04 = 0.0f;

	/** Y coefficient of the r^2*cos(2*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CY22 = 0.0f;

	/** Y coefficient of the r^4*cos(2*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CY24 = 0.0f;

	/** Y coefficient of the r^4*cos(4*phi) term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float CY44 = 0.0f;

	/** Squeeze Factor (should be small, relatively close to 1.0) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float SqueezeX = 1.0f;

	/** Squeeze Factor (should be small, relatively close to 1.0) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float SqueezeY = 1.0f;

	/** Lens Rotation in degrees. Represents mounting inaccuracies (should be small, between -2 and +2 degrees) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float LensRotation = 0.0f;
};

/**
 * Anamorphic lens model, using Anamorphic lens distortion parameters
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Anamorphic Lens Model"))
class UAnamorphicLensModel : public ULensModel
{
	GENERATED_BODY()

public:
	//~ Begin ULensModel interface
 	UE_API virtual UScriptStruct* GetParameterStruct() const override;
 	UE_API virtual FName GetModelName() const override;
 	UE_API virtual FName GetShortModelName() const override;
	//~ End ULensModel interface
};

#undef UE_API
