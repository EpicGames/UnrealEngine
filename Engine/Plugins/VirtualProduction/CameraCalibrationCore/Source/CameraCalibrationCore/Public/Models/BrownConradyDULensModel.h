// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensModel.h"
#include "BrownConradyUDLensModel.h"

#include "BrownConradyDULensModel.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/**
 * Brown-Conrady D-U lens model (Distorted to Undistorted direction)
 * Uses the same parameters as Brown-Conrady U-D but interprets them in the opposite direction.
 * The polynomial division model represents the mapping from distorted to undistorted coordinates.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Brown-Conrady D-U Lens Model"))
class UBrownConradyDULensModel : public ULensModel
{
	GENERATED_BODY()

public:
	//~ Begin ULensModel interface
	UE_API virtual UScriptStruct* GetParameterStruct() const override;
	UE_API virtual FName GetModelName() const override;
	UE_API virtual FName GetShortModelName() const override;
	//~ End ULensModel interface
};

#undef UE_API