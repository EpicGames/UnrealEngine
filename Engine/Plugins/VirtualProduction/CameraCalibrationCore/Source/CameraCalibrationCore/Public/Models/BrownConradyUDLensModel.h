// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensModel.h"

#include "BrownConradyUDLensModel.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/**
 * Brown-Conrady U-D lens distortion parameters (polynomial division model), using
 * a polynomial division model: (1 + K1*r^2 + K2*r^4 + K3*r^6) / (1 + K4*r^2 + K5*r^4 + K6*r^6)
 * where the output of that equation represents the distorted image coordinates.
 */
USTRUCT(BlueprintType)
struct FBrownConradyUDDistortionParameters
{
	GENERATED_BODY()

public:
	/** Radial coefficient of the r^2 term (numerator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K1 = 0.0f;

	/** Radial coefficient of the r^4 term (numerator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K2 = 0.0f;

	/** Radial coefficient of the r^6 term (numerator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K3 = 0.0f;

	/** Radial coefficient of the r^2 term (denominator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K4 = 0.0f;

	/** Radial coefficient of the r^4 term (denominator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K5 = 0.0f;

	/** Radial coefficient of the r^6 term (denominator) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K6 = 0.0f;

	/** First tangential coefficient */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float P1 = 0.0f;

	/** Second tangential coefficient */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float P2 = 0.0f;
};

/**
 * Brown-Conrady U-D lens model, using Brown-Conrady U-D lens distortion parameters
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Brown-Conrady U-D Lens Model"))
class UBrownConradyUDLensModel : public ULensModel
{
	GENERATED_BODY()

public:
	//~ Begin ULensModel interface
	UE_API virtual UScriptStruct* GetParameterStruct() const override;
	UE_API virtual FName GetModelName() const override;
	UE_API virtual FName GetShortModelName() const override;
	//~ End ULensModel interface
};

#undef UE_API