// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/StructOnScope.h"

#include "LensModel.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API


class ULensDistortionModelHandlerBase;


/**
 * Abstract base class for lens models
 */
UCLASS(MinimalAPI, Abstract)
class ULensModel : public UObject
{
	GENERATED_BODY()

public:
	/** Get the lens model name */
	virtual FName GetModelName() const PURE_VIRTUAL(ULensModel::GetModelName, return FName(""););

	/** Get the lens model short name */
	virtual FName GetShortModelName() const PURE_VIRTUAL(ULensModel::GetShortModelName, return FName(""););

	/** Get the struct of distortion parameters supported by this model */
	virtual UScriptStruct* GetParameterStruct() const PURE_VIRTUAL(ULensModel::GetParameterStruct, return nullptr;);

	/** Returns an array of floats representing the default values of each property in the parameter struct */
	UE_API void GetDefaultParameterArray(TArray<float>& OutParameterValues) const;

#if WITH_EDITOR
	/** Get the names of each float parameters supported by this model */
	UE_API virtual TArray<FText> GetParameterDisplayNames() const;
#endif //WITH_EDITOR

	/** Get the number of float fields in the parameter struct supported by this model */
	UE_API virtual uint32 GetNumParameters() const;

	/** 
	 * Fill the destination array of floats with the values of the fields in the source struct 
	 * Note: the template type must be a UStruct
	 */
	template<typename StructType>
	void ToArray(const StructType& SrcData, TArray<float>& DstArray) const
	{
		ToArray_Internal(StructType::StaticStruct(), &SrcData, DstArray);
	}

	/** 
	 * ToArray specialization taking a StructOnScope containing type and data 
	 */
	template<>
	void ToArray<FStructOnScope>(const FStructOnScope& SrcData, TArray<float>& DstArray) const
	{
		ToArray_Internal(static_cast<const UScriptStruct*>(SrcData.GetStruct()), reinterpret_cast<const void*>(SrcData.GetStructMemory()), DstArray);
	}

	/**
	 * Populate the float fields in the destination struct with the values in the source array
	 * Note: the template type must be a UStruct
	 */
	template<typename StructType>
	void FromArray(const TArray<float>& SrcArray, StructType& DstData)
	{
		FromArray_Internal(StructType::StaticStruct(), SrcArray, &DstData);
	}

	/** Returns the first handler that supports the given LensModel */
	static UE_API TSubclassOf<ULensDistortionModelHandlerBase> GetHandlerClass(TSubclassOf<ULensModel> LensModel);

protected:
	/** Internal implementation of ToArray. See declaration of public template method. */
	UE_API virtual void ToArray_Internal(const UScriptStruct* TypeStruct, const void* SrcData, TArray<float>& DstArray) const;

	/** Internal implementation of FromArray. See declaration of public template method. */
	UE_API virtual void FromArray_Internal(UScriptStruct* TypeStruct, const TArray<float>& SrcArray, void* DstData);
};

#undef UE_API
