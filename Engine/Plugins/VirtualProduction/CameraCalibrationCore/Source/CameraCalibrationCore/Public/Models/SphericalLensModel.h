// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensModel.h"

#include "SphericalLensModel.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/**
 * Spherical lens distortion parameters
 * All parameters are unitless and represent the coefficients used to undistort a distorted image
 * Refer to the OpenCV camera calibration documentation for the intended units/usage of these parameters:
 * https://docs.opencv.org/3.4/d9/d0c/group__calib3d.html
 */
USTRUCT(BlueprintType)
struct FSphericalDistortionParameters
{
	GENERATED_BODY()

public:
	/** Radial coefficient of the r^2 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K1 = 0.0f;

	/** Radial coefficient of the r^4 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K2 = 0.0f;

	/** Radial coefficient of the r^6 term */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float K3 = 0.0f;

	/** First tangential coefficient */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float P1 = 0.0f;

	/** Second tangential coefficient */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Distortion")
	float P2 = 0.0f;
};

/**
 * Spherical lens model, using spherical lens distortion parameters
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Spherical Lens Model"))
class USphericalLensModel : public ULensModel
{
	GENERATED_BODY()

public:
	//~ Begin ULensModel interface
	UE_API virtual UScriptStruct* GetParameterStruct() const override;
	UE_API virtual FName GetModelName() const override;
	UE_API virtual FName GetShortModelName() const override;
	//~ End ULensModel interface
};

#undef UE_API
