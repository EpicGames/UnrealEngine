// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"

class UTextureRenderTarget2D;

/** Parameters for the crosshair shader */
struct FCrosshairOverlayParams
{
	/** Normalized image center of the lens where the crosshair will be centered */
	FVector2f PrincipalPoint = { 0.5f, 0.5f };

	/** Length of one side of the crosshair (in pixels) */
	float LengthInPixels = 50.0f;

	/** Size of the gap in the center of the crosshair (in pixels) */
	float GapSizeInPixels = 2.0f;

	/** Color to draw the crosshair */
	FVector4f CrosshairColor = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
};


namespace OverlayRendering
{
	/** Draws a crosshair overlay to the output render target */
	CAMERACALIBRATIONCORE_API bool DrawCrosshairOverlay(UTextureRenderTarget2D* OutRenderTarget, const FCrosshairOverlayParams& CrosshairParams);
}
