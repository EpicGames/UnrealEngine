// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LensDistortionModelHandlerBase.h"

#include "Models/SphericalLensModel.h"

#include "SphericalLensDistortionModelHandler.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

/** Lens distortion handler for a spherical lens model that implements the Brown-Conrady polynomial model */
UCLASS(MinimalAPI, BlueprintType)
class USphericalLensDistortionModelHandler : public ULensDistortionModelHandlerBase
{
	GENERATED_BODY()

protected:
	//~ Begin ULensDistortionModelHandlerBase interface
	UE_API virtual void InitializeHandler() override;
	UE_API virtual FVector2D ComputeDistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual FVector2D ComputeUndistortedUV(const FVector2D& InScreenUV) const override;
	UE_API virtual void InitDistortionMaterials() override;
	UE_API virtual void UpdateMaterialParameters() override;
	UE_API virtual void InterpretDistortionParameters() override;
	UE_API virtual FString GetDistortionShaderPath() const override;
	UE_API virtual void ExecuteDistortionShader(class FRDGBuilder& GraphBuilder, const FLensDistortionState& InCurrentState, float InverseOverscan, float CameraOverscan, const FVector2D& SensorSize, FRDGTextureRef& OutDistortionMap) const override;
	//~ End ULensDistortionModelHandlerBase interface

private:
	/** Spherical lens distortion parameters (k1, k2, k3, p1, p2) */
	FSphericalDistortionParameters SphericalParameters;
};

#undef UE_API
