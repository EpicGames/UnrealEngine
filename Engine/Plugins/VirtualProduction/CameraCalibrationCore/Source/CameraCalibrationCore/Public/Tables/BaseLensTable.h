// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"

#include "UObject/WeakObjectPtrTemplates.h"

#include "BaseLensTable.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

struct FKeyHandle;
struct FRichCurve;
class ULensFile;
enum class ELensDataCategory : uint8;

/**
 * Extra information about linked points
 */
struct FLinkPointMetadata
{
	FLinkPointMetadata() = default;

	FLinkPointMetadata(const bool bInRemoveByDefault)
		: bRemoveByDefault(bInRemoveByDefault)
	{}

	/** Whether the linked point should be set to remove by default */ 
	bool bRemoveByDefault = true;
};

/**
 * Base focus point struct
 */
USTRUCT()
struct FBaseFocusPoint
{
	GENERATED_BODY()

public:
	virtual ~FBaseFocusPoint() = default;

	/** Returns focus value for this Focus Point */
	virtual float GetFocus() const PURE_VIRTUAL(FBaseLensTable::GetFocus, return 0.f;);

	/** Returns number of zoom points */
	virtual int32 GetNumPoints() const PURE_VIRTUAL(FBaseLensTable::GetNumPoints, return 0;);

	/** Returns zoom value for a given index */
	virtual float GetZoom(int32 Index) const PURE_VIRTUAL(FBaseLensTable::GetZoom, return 0.f;);
};

template<>
struct TStructOpsTypeTraits<FBaseFocusPoint> : public TStructOpsTypeTraitsBase2<FBaseFocusPoint>
{
	enum
	{
		WithPureVirtual = true,
	};
};

/** Base focus curve struct */
USTRUCT()
struct FBaseFocusCurve
{
	GENERATED_BODY()

protected:
	/** Adds a new key to the specified curve */
	FKeyHandle AddPointToCurve(FRichCurve& InCurve, float InFocus, float InValue, float InputTolerance, FKeyHandle InOptionalKeyHandle = FKeyHandle());
	
	/** Sets the value of an existing key in the specified curve */
	FKeyHandle SetPointInCurve(FRichCurve& InCurve, float InFocus, float InValue, float InputTolerance);
	
	/** Deletes a key at the specified focus from the specified curve */
	void DeletePointFromCurve(FRichCurve& InCurve, float InFocus, float InputTolerance);
	
	/** Changes the focus of a key in the specified curve */
	void ChangeFocusInCurve(FRichCurve& InCurve, float InExistingFocus, float InNewFocus, float InputTolerance);
	
	/** Changes the focus of a key in the specified curve and optionally replaces any key that already exists at the new focus */
	void MergeFocusInCurve(FRichCurve& InCurve, float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance);
};

/**
 * Base data table struct
 */
USTRUCT()
struct FBaseLensTable
{
	GENERATED_BODY()

	friend ULensFile;

	/** Callback to get the base focus point reference */
	using FFocusPointCallback = TFunction<void(const FBaseFocusPoint& /*InFocusPoint*/)>;

	/** Callback to get the linked focus point reference */
	using FLinkedFocusPointCallback = TFunction<void(const FBaseFocusPoint& /*InFocusPoint*/, ELensDataCategory /*Category*/, FLinkPointMetadata /* InPointMeta*/)>;
	
protected:

	/** Returns the map of linked categories  */
	virtual TMap<ELensDataCategory, FLinkPointMetadata> GetLinkedCategories() const PURE_VIRTUAL(FBaseLensTable::GetLinkedCategories, return TMap<ELensDataCategory, FLinkPointMetadata>() ;);

	/**
	 * Whether the focus point exists
	 * @param InFocus focus value to check
	 * @return true if the point exists
	 */
	virtual bool DoesFocusPointExists(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const PURE_VIRTUAL(FBaseLensTable::DoesFocusPointExists, return false; );

	/**
	 * Whether the zoom point exists
	 * @param InFocus focus value to check 
	 * @param InZoom zoom value to check
	 * @param InputTolerance Maximum allowed difference for considering them as 'nearly equal'
	 * @return true if zoom point exists
	 */
	virtual bool DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) const PURE_VIRTUAL(FBaseLensTable::DoesZoomPointExists, return false; );

	/** Copies the specified keys from the source curve to the destination curve */
	UE_API void CopyCurveKeys(const FRichCurve& InSourceCurve, FRichCurve& InDestCurve, TArrayView<const FKeyHandle> InKeys);

	/** Propagates the values of a curve to a set of cross curves at the specified time */
	UE_API void PropagateCurveValuesToCrossCurves(const FRichCurve& InCurve, float InCrossCurveTime, TFunctionRef<FRichCurve*(float)> GetCurveFn);
	
public:
	virtual ~FBaseLensTable() = default;

	/**
	 * Loop through all Focus Points
	 * @param InCallback Callback with Focus point reference
	 */
	virtual void ForEachPoint(FFocusPointCallback InCallback) const PURE_VIRTUAL(FBaseLensTable::ForEachPoint );

	/** Get number of Focus points for this data table */
	virtual int32 GetFocusPointNum() const PURE_VIRTUAL(FBaseLensTable::GetFocusPointNum, return INDEX_NONE; );

	/** Get total number of Zoom points for all Focus points of this data table */
	virtual int32 GetTotalPointNum() const PURE_VIRTUAL(FBaseLensTable::GetTotalPointNum, return INDEX_NONE; );

	/** Get the base focus point by given index */
	virtual const FBaseFocusPoint* GetBaseFocusPoint(int32 InIndex) const PURE_VIRTUAL(FBaseLensTable::GetBaseFocusPoint, return nullptr; );

	/** Get Struct class of this Data Table */
	virtual UScriptStruct* GetScriptStruct() const PURE_VIRTUAL(FBaseLensTable::GetFocusPointNum, return nullptr; );

	/** 
	* Fills OutCurve with all points contained in the given focus 
	* Returns false if FocusIdentifier is not found or ParameterIndex isn't valid
	*/
	virtual bool BuildParameterCurveAtFocus(float InFocus, int32 ParameterIndex, FRichCurve& OutCurve) const PURE_VIRTUAL(FBaseLensTable::BuildParameterCurve, return false; ); 

	/**
	 * Fills OutCurve with points across all focuses that have the given zoom
	 * @param InZoom The zoom to get the curve for
	 * @param ParameterIndex The index of the data parameter to get the curve for
	 * @param OutCurve The curve to fill
	 * @return true if the curve was built, otherwise false
	 */
	virtual bool BuildParameterCurveAtZoom(float InZoom, int32 ParameterIndex, FRichCurve& OutCurve) const PURE_VIRTUAL(FBaseLensTable::BuildFocusParameterCurve, return false; );

	/**
	 * Updates the keys of all zoom points at the specified focus to match the corresponding keys in the specified curve
	 * @param InFocus The focus whose zoom points should be updated
	 * @param InParameterIndex The index of the parameter whose curve keys are being updated
	 * @param InSourceCurve The curve to copy the keys from
	 * @param InKeys The keys to copy from the source curve into the zoom parameter curve
	 */
	virtual void SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) PURE_VIRTUAL(FBaseLensTable::SetZoomParameterCurveKeys);

	/**
	 * Updates the keys of all focus points at the specified zoom to match the corresponding keys in the specified curve
	 * @param InZoom The zoom whose focus points should be updated
	 * @param InParameterIndex The index of the parameter whose curve keys are being updated
	 * @param InSourceCurve The curve to copy the keys from
	 * @param InKeys The keys to copy from the source curve into the focus parameter curve
	 */
	virtual void SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) PURE_VIRTUAL(FBaseLensTable::SetFocusParameterCurveKeys);

	/**
	 * Gets whether the positions of the table's curve keys can be edited or not
	 * @param InParameterIndex The index of the parameter of the curve in question
	 * @return true if the curve's key positions can be edited, false otherwise
	 */
	virtual bool CanEditCurveKeyPositions(int32 InParameterIndex) const PURE_VIRTUAL(FBaseLensTable::CanEditCurveKeyPositions, return false; );

	/**
	 * Gets whether the attributes of the table's curve keys can be edited or not
	 * @param InParameterIndex The index of the parameter of the curve in question
	 * @return true if the curve's key attributes can be edited, false otherwise
	 */
	virtual bool CanEditCurveKeyAttributes(int32 InParameterIndex) const PURE_VIRTUAL(FBaseLensTable::CanEditCurveKeyAttributes, return false; );

	/**
	 * Gets the range of allowed values for the curve keys at the specified parameter index
	 * @param InParameterIndex The index of the parameter to get the ranges for
	 * @return The minimum and maximum allowed values
	 */
	virtual TRange<double> GetCurveKeyPositionRange(int32 InParameterIndex) const { return TRange<double>(TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max()); }
	
	/** Gets the text to display on any UI when labeling the values of this table's parameters */
	virtual FText GetParameterValueLabel(int32 InParameterIndex) const { return FText(); }

	/** Gets the text to display on any UI when displaying units of the values of this table's parameters */
	virtual FText GetParameterValueUnitLabel(int32 InParameterIndex) const { return FText(); }
	
	/** Get Names of this Data Point */
	static UE_API FName GetFriendlyPointName(ELensDataCategory InCategory);

	/**
	 * Loop through all Focus Points base on given focus value
	 * @param InCallback Callback with Focus point reference
	 * @param InFocus focus value to check 
	 * @param InputTolerance Maximum allowed difference for considering them as 'nearly equal'
	 */
	UE_API void ForEachFocusPoint(FFocusPointCallback InCallback, const float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/**
	 * Loop through all linked Focus Points base on given focus value
	 * @param InCallback Callback with Focus point reference, category and link meta
	 * @param InFocus focus value to check 
	 * @param InputTolerance Maximum allowed difference for considering them as 'nearly equal'
	 */
	UE_API void ForEachLinkedFocusPoint(FLinkedFocusPointCallback InCallback, const float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/**
	 * Whether the linkage exists for given focus value
	 * @param InFocus focus value to check 
	 * @param InputTolerance Maximum allowed difference for considering them as 'nearly equal'
	 * @return true if linkage exists
	 */
	UE_API bool HasLinkedFocusValues(const float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;

	 /**
	 * Whether the linkage exists for given focus and zoom values
	 * @param InFocus focus value to check 
	 * @param InZoomPoint zoom value to check
	 * @param InputTolerance Maximum allowed difference for considering them as 'nearly equal'
	 * @return true if linkage exists
	 */
	UE_API bool HasLinkedZoomValues(const float InFocus, const float InZoomPoint, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/**
	 * Whether given value fit between Focus Point Neighbors
	 * @param InFocusPoint given focus point
	 * @param InFocusValueToEvaluate value to evaluate between focus point neighbors
	 * @return true if value fit between neighbors
	 */
	UE_API bool IsFocusBetweenNeighbor(const float InFocusPoint, const float InFocusValueToEvaluate) const;

	/** Get the pointer to owner lens file */
	ULensFile* GetLensFile() const { return LensFile.Get(); }
	
private:
	/**
	 * Lens file owner reference
	 */
	UPROPERTY()
	TWeakObjectPtr<ULensFile> LensFile;
};

template<>
struct TStructOpsTypeTraits<FBaseLensTable> : public TStructOpsTypeTraitsBase2<FBaseLensTable>
{
	enum
	{
		WithPureVirtual = true,
	};
};

#undef UE_API
