// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "LensData.h"
#include "Tables/BaseLensTable.h"

#include "DistortionParametersTable.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API


/**
 * Distortion parameters associated to a zoom value
 */
USTRUCT()
struct FDistortionZoomPoint
{
	GENERATED_BODY()

public:

	/** Input zoom value for this point */
	UPROPERTY()
	float Zoom = 0.0f;

	/** Distortion parameters for this point */
	UPROPERTY(EditAnywhere, Category = "Distortion")
	FDistortionInfo DistortionInfo;
};

/**
 * Contains list of distortion parameters points associated to zoom value
 */
USTRUCT()
struct FDistortionFocusPoint : public FBaseFocusPoint
{
	GENERATED_BODY()

	using PointType = FDistortionInfo;
	
public:
	//~ Begin FBaseFocusPoint Interface
	virtual float GetFocus() const override { return Focus; }
	UE_API virtual int32 GetNumPoints() const override;
	UE_API virtual float GetZoom(int32 Index) const override;
	//~ End FBaseFocusPoint Interface

	UE_API void RemovePoint(float InZoomValue);

	/** Returns data type copy value for a given float */
	UE_API bool GetPoint(float InZoom, FDistortionInfo& OutData, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Adds a new point at InZoom. Updates existing one if tolerance is met */
	UE_API bool AddPoint(float InZoom, const FDistortionInfo& InData, float InputTolerance, bool bIsCalibrationPoint);

	/** Sets an existing point at InZoom. Updates existing one if tolerance is met */
	UE_API bool SetPoint(float InZoom, const FDistortionInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Gets whether the point at InZoom is a calibration point. */
	bool IsCalibrationPoint(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) { return false; }
	
	/** Returns true if this point is empty */
	UE_API bool IsEmpty() const;

	/** Gets the curve for the specified parameter, or nullptr if the parameter index is invalid */
	UE_API const FRichCurve* GetCurveForParameter(int32 InParameterIndex) const;
	
	UE_API void SetParameterValue(int32 InZoomIndex, float InZoomValue, int32 InParameterIndex, float InParameterValue);

public:

	/** Input focus value for this point */
	UPROPERTY()
	float Focus = 0.0f;

	/** Curves describing desired blending between resulting displacement maps */
	UPROPERTY()
	FRichCurve MapBlendingCurve;

	/** List of zoom points */
	UPROPERTY()
	TArray<FDistortionZoomPoint> ZoomPoints;
};

/** A curve along the focus axis for a single zoom value */
USTRUCT()
struct FDistortionFocusCurve : public FBaseFocusCurve
{
	GENERATED_BODY()

public:
	/** Adds a new point to the focus curve, or updates a matching existing point if one is found */
	UE_API void AddPoint(float InFocus, const FDistortionInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Updates an existing point if one is found */
	UE_API void SetPoint(float InFocus, const FDistortionInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Removes the point at the specified focus if one is found */
	UE_API void RemovePoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the focus value of the point at the specified focus, if one is found */
	UE_API void ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the focus value of the point at the specified focus and optionally replaces any point at the new focus with the old point */
	UE_API void MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Gets whether the curve is empty */
	UE_API bool IsEmpty() const;

	/** Gets the curve for the specified parameter, or nullptr if the parameter index is invalid */
	UE_API const FRichCurve* GetCurveForParameter(int32 InParameterIndex) const;
	
public:
	/** Curve describing desired blending between resulting displacement maps */
	UPROPERTY()
	FRichCurve MapBlendingCurve;

	/** The fixed zoom value of the curve */
	UPROPERTY()
	float Zoom = 0.0f;
};

/**
 * Distortion table containing list of points for each focus and zoom input
 */
USTRUCT()
struct FDistortionTable : public FBaseLensTable
{
	GENERATED_BODY()

	using FocusPointType = FDistortionFocusPoint;
	using FocusCurveType = FDistortionFocusCurve;

	/** Wrapper for indices of specific parameters for the distortion table  */
	struct FParameters
	{
		static constexpr int32 Aggregate = INDEX_NONE;
	};
	
protected:
	//~ Begin FBaseDataTable Interface
	UE_API virtual TMap<ELensDataCategory, FLinkPointMetadata> GetLinkedCategories() const override;
	UE_API virtual bool DoesFocusPointExists(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const override;
	UE_API virtual bool DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) const override;
	UE_API virtual const FBaseFocusPoint* GetBaseFocusPoint(int32 InIndex) const override;
	//~ End FBaseDataTable Interface
	
public:
	//~ Begin FBaseDataTable Interface
	UE_API virtual void ForEachPoint(FFocusPointCallback InCallback) const override;
	virtual int32 GetFocusPointNum() const override { return FocusPoints.Num(); }
	UE_API virtual int32 GetTotalPointNum() const override;
	UE_API virtual UScriptStruct* GetScriptStruct() const override;
	UE_API virtual bool BuildParameterCurveAtFocus(float InFocus, int32 InParameterIndex, FRichCurve& OutCurve) const override;
	UE_API virtual bool BuildParameterCurveAtZoom(float InZoom, int32 InParameterIndex, FRichCurve& OutCurve) const override;
	UE_API virtual void SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) override;
	UE_API virtual void SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) override;
	UE_API virtual bool CanEditCurveKeyPositions(int32 InParameterIndex) const override;
	UE_API virtual bool CanEditCurveKeyAttributes(int32 InParameterIndex) const override;
	UE_API virtual FText GetParameterValueLabel(int32 InParameterIndex) const override;
	//~ End FBaseDataTable Interface

	/** Returns const point for a given focus */
	UE_API const FDistortionFocusPoint* GetFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;
	
	/** Returns point for a given focus */
	UE_API FDistortionFocusPoint* GetFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Gets the focus curve for the specified zoom, or nullptr if none were found */
	UE_API const FDistortionFocusCurve* GetFocusCurve(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Gets the focus curve for the specified zoom, or nullptr if none were found */
	UE_API FDistortionFocusCurve* GetFocusCurve(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Returns all focus points */
	UE_API TConstArrayView<FDistortionFocusPoint> GetFocusPoints() const;

	/** Returns all focus points */
	UE_API TArray<FDistortionFocusPoint>& GetFocusPoints();

	/** Returns all focus curves */
	UE_API TConstArrayView<FDistortionFocusCurve> GetFocusCurves() const;

	/** Returns all focus curves */
	UE_API TArray<FDistortionFocusCurve>& GetFocusCurves();
	
	/** Removes a focus point identified as InFocusIdentifier */
	UE_API void RemoveFocusPoint(float InFocus);

	/** Checks to see if there exists a focus point matching the specified focus value */
	UE_API bool HasFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;
	
	/** Changes the value of a focus point */
	UE_API void ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Merges the points in the specified source focus into the specified destination focus */
	UE_API void MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Removes a zoom point from a focus point*/
	UE_API void RemoveZoomPoint(float InFocus, float InZoom);

	/** Checks to see if there exists a zoom point matching the specified zoom and focus values */
	UE_API bool HasZoomPoint(float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the value of a zoom point */
	UE_API void ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Adds a new point in the table */
	UE_API bool AddPoint(float InFocus, float InZoom, const FDistortionInfo& InData, float InputTolerance, bool bIsCalibrationPoint);

	/** Get the point from the table */
	UE_API bool GetPoint(const float InFocus, const float InZoom, FDistortionInfo& OutData, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Set a new point into the table */
	UE_API bool SetPoint(float InFocus, float InZoom, const FDistortionInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Builds the focus curves to match existing data in the table */
	UE_API void BuildFocusCurves();
	
public:

	/** Lists of focus points */
	UPROPERTY()
	TArray<FDistortionFocusPoint> FocusPoints;

	/** A list of curves along the focus axis for each zoom value */
	UPROPERTY()
	TArray<FDistortionFocusCurve> FocusCurves;
};

#undef UE_API
