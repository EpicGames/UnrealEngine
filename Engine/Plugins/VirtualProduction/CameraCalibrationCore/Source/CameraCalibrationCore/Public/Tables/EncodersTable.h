// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "LensData.h"
#include "Tables/BaseLensTable.h"

#include "EncodersTable.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API


/**
 * Encoder table containing mapping from raw input value to nominal value
 */
USTRUCT()
struct FEncodersTable
{
	GENERATED_BODY()

public:
	/** Returns number of focus points */
	UE_API int32 GetNumFocusPoints() const;

	/** Returns raw focus value for a given index */
	UE_API float GetFocusInput(int32 Index) const;
	
	/** Returns focus value for a given index */
	UE_API float GetFocusValue(int32 Index) const;

	/** Removes a focus mapping point at the given raw input */
	UE_API bool RemoveFocusPoint(float InRawFocus);

	/** Returns number of focus points */
	UE_API int32 GetNumIrisPoints() const;

	/** Returns  value for a given index */
	UE_API float GetIrisInput(int32 Index) const;
	
	/** Returns zoom value for a given index */
	UE_API float GetIrisValue(int32 Index) const;	

	/** Removes a iris mapping point at the given raw input */
	UE_API bool RemoveIrisPoint(float InRawIris);

	/** Removes all focus and iris points */
	UE_API void ClearAll();


public:
	/** Focus curve from encoder values to nominal values */
	UPROPERTY()
	FRichCurve Focus;

	/** Iris curve from encoder values to nominal values */
	UPROPERTY()
	FRichCurve Iris;
};

#undef UE_API
