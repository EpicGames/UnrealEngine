// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"
#include "LensData.h"
#include "Tables/BaseLensTable.h"

#include "STMapTable.generated.h"

#define UE_API CAMERACALIBRATIONCORE_API

class UTextureRenderTarget2D;

/**
 * Derived data computed from parameters or stmap
 */
USTRUCT()
struct FDerivedDistortionData
{
	GENERATED_BODY()

	/** Precomputed data about distortion */
	UPROPERTY(VisibleAnywhere, Category = "Distortion")
	FDistortionData DistortionData;

	/** Computed displacement map based on undistortion data */
	UPROPERTY(Transient, VisibleAnywhere, Category = "Distortion")
	TObjectPtr<UTextureRenderTarget2D> UndistortionDisplacementMap = nullptr;

	/** Computed displacement map based on distortion data */
	UPROPERTY(Transient, VisibleAnywhere, Category = "Distortion")
	TObjectPtr<UTextureRenderTarget2D> DistortionDisplacementMap = nullptr;

	/** When dirty, derived data needs to be recomputed */
	bool bIsDirty = true;
};

/**
 * STMap data associated to a zoom input value
 */
USTRUCT()
struct FSTMapZoomPoint
{
	GENERATED_BODY()

public:

	/** Input zoom value for this point */
	UPROPERTY()
	float Zoom = 0.0f;

	/** Data for this zoom point */
	UPROPERTY()
	FSTMapInfo STMapInfo;

	/** Derived distortion data associated with this point */
	UPROPERTY(Transient)
	FDerivedDistortionData DerivedDistortionData;

	/** Whether this point was added in calibration along distortion */
	UPROPERTY()
	bool bIsCalibrationPoint = false;
};

/**
 * A data point associating focus and zoom to lens parameters
 */
USTRUCT()
struct FSTMapFocusPoint : public FBaseFocusPoint
{
	GENERATED_BODY()

	using PointType = FSTMapInfo;
	
public:
	//~ Begin FBaseFocusPoint Interface
	virtual float GetFocus() const override { return Focus; }
	UE_API virtual int32 GetNumPoints() const override;
	UE_API virtual float GetZoom(int32 Index) const override;
	//~ End FBaseFocusPoint Interface
	
	/** Returns const point for a given zoom */
	UE_API const FSTMapZoomPoint* GetZoomPoint(float InZoom) const;

	/** Returns point for a given focus */
	UE_API FSTMapZoomPoint* GetZoomPoint(float InZoom);

	/** Returns zoom value for a given float */
	UE_API bool GetPoint(float InZoom, FSTMapInfo& OutData, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Adds a new point at InZoom. Updates existing one if tolerance is met */
	UE_API bool AddPoint(float InZoom, const FSTMapInfo& InData, float InputTolerance, bool bIsCalibrationPoint);

	/** Sets an existing point at InZoom. Updates existing one if tolerance is met */
	UE_API bool SetPoint(float InZoom, const FSTMapInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Gets whether the point at InZoom is a calibration point. */
	UE_API bool IsCalibrationPoint(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Removes a point corresponding to specified zoom */
	UE_API void RemovePoint(float InZoomValue);

	/** Returns true if this point is empty */
	UE_API bool IsEmpty() const;
	
public:

	/** Input focus for this point */
	UPROPERTY()
	float Focus = 0.0f;

	/** Curve used to blend displacement map together to give user more flexibility */
	UPROPERTY()
	FRichCurve MapBlendingCurve;

	/** Zoom points for this focus */
	UPROPERTY()
	TArray<FSTMapZoomPoint> ZoomPoints;
};

/** A curve along the focus axis for a single zoom value */
USTRUCT()
struct FSTMapFocusCurve : public FBaseFocusCurve
{
	GENERATED_BODY()

public:
	/** Adds a new point to the focus curve, or updates a matching existing point if one is found */
	UE_API void AddPoint(float InFocus, const FSTMapInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Updates an existing point if one is found */
	UE_API void SetPoint(float InFocus, const FSTMapInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Removes the point at the specified focus if one is found */
	UE_API void RemovePoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the focus value of the point at the specified focus, if one is found */
	UE_API void ChangeFocus(float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the focus value of the point at the specified focus and optionally replaces any point at the new focus with the old point */
	UE_API void MergeFocus(float InExistingFocus, float InNewFocus, bool bReplaceExisting, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Gets whether the curve is empty */
	UE_API bool IsEmpty() const;

public:
	/** Curve describing desired blending between resulting displacement maps */
	UPROPERTY()
	FRichCurve MapBlendingCurve;

	/** The fixed zoom value of the curve */
	UPROPERTY()
	float Zoom = 0.0f;
};

/**
 * STMap table containing list of points for each focus and zoom inputs
 */
USTRUCT()
struct FSTMapTable : public FBaseLensTable
{
	GENERATED_BODY()

	using FocusPointType = FSTMapFocusPoint;
	using FocusCurveType = FSTMapFocusCurve;

protected:
	//~ Begin FBaseDataTable Interface
	UE_API virtual TMap<ELensDataCategory, FLinkPointMetadata> GetLinkedCategories() const override;
	UE_API virtual bool DoesFocusPointExists(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const override;
	UE_API virtual bool DoesZoomPointExists(float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) const override;
	UE_API virtual const FBaseFocusPoint* GetBaseFocusPoint(int32 InIndex) const override;
	//~ End FBaseDataTable Interface
	
public:
	//~ Begin FBaseDataTable Interface
	UE_API virtual void ForEachPoint(FFocusPointCallback InCallback) const override;
	virtual int32 GetFocusPointNum() const override { return FocusPoints.Num(); }
	UE_API virtual int32 GetTotalPointNum() const override;
	UE_API virtual UScriptStruct* GetScriptStruct() const override;
	UE_API virtual bool BuildParameterCurveAtFocus(float InFocus, int32 ParameterIndex, FRichCurve& OutCurve) const override;
	UE_API virtual bool BuildParameterCurveAtZoom(float InZoom, int32 ParameterIndex, FRichCurve& OutCurve) const override;
	UE_API virtual void SetParameterCurveKeysAtFocus(float InFocus, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) override;
	UE_API virtual void SetParameterCurveKeysAtZoom(float InZoom, int32 InParameterIndex, const FRichCurve& InSourceCurve, TArrayView<const FKeyHandle> InKeys) override;
	virtual bool CanEditCurveKeyPositions(int32 InParameterIndex) const override { return false; }
	virtual bool CanEditCurveKeyAttributes(int32 InParameterIndex) const override { return true; }
	//~ End FBaseDataTable Interface

	/** Returns const point for a given focus */
	UE_API const FSTMapFocusPoint* GetFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Returns point for a given focus */
	UE_API FSTMapFocusPoint* GetFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Gets the focus curve for the specified zoom, or nullptr if none were found */
	UE_API const FSTMapFocusCurve* GetFocusCurve(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Gets the focus curve for the specified zoom, or nullptr if none were found */
	UE_API FSTMapFocusCurve* GetFocusCurve(float InZoom, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Returns all focus points */
	UE_API TConstArrayView<FSTMapFocusPoint> GetFocusPoints() const;

	/** Returns all focus points */
	UE_API TArrayView<FSTMapFocusPoint> GetFocusPoints();

	/** Returns all focus curves */
	UE_API TConstArrayView<FSTMapFocusCurve> GetFocusCurves() const;

	/** Returns all focus curves */
	UE_API TArray<FSTMapFocusCurve>& GetFocusCurves();
	
	/** Removes a focus point identified as InFocusIdentifier */
	UE_API void RemoveFocusPoint(float InFocus);

	/** Checks to see if there exists a focus point matching the specified focus value */
	UE_API bool HasFocusPoint(float InFocus, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Changes the value of a focus point */
	UE_API void ChangeFocusPoint(float InExistingFocus, float InNewFocus, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Merges the points in the specified source focus into the specified destination focus */
	UE_API void MergeFocusPoint(float InSrcFocus, float InDestFocus, bool bReplaceExistingZoomPoints, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Removes a zoom point from a focus point*/
	UE_API void RemoveZoomPoint(float InFocus, float InZoom);

	/** Checks to see if there exists a zoom point matching the specified zoom and focus values */
	UE_API bool HasZoomPoint(float InFocus, float InZoom, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Changes the value of a zoom point */
	UE_API void ChangeZoomPoint(float InFocus, float InExistingZoom, float InNewZoom, float InputTolerance = KINDA_SMALL_NUMBER);
	
	/** Adds a new point in the table */
	UE_API bool AddPoint(float InFocus, float InZoom, const FSTMapInfo& InData,  float InputTolerance, bool bIsCalibrationPoint);

	/** Get the point from the table */
	UE_API bool GetPoint(const float InFocus, const float InZoom, FSTMapInfo& OutData, float InputTolerance = KINDA_SMALL_NUMBER) const;

	/** Set a new point into the table */
	UE_API bool SetPoint(float InFocus, float InZoom, const FSTMapInfo& InData, float InputTolerance = KINDA_SMALL_NUMBER);

	/** Builds the focus curves to match existing data in the table */
	UE_API void BuildFocusCurves();
	
public:

	/** Lists of focus points */
	UPROPERTY()
	TArray<FSTMapFocusPoint> FocusPoints;

	/** A list of curves along the focus axis for each zoom value */
	UPROPERTY()
	TArray<FSTMapFocusCurve> FocusCurves;
};

#undef UE_API
