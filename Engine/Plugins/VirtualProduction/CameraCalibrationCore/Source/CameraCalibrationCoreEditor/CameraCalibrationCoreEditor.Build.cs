// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CameraCalibrationCoreEditor : ModuleRules
	{
		public CameraCalibrationCoreEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CameraCalibrationCore",
					"Core",
					"CoreUObject",
					"Engine",
					"PlacementMode",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"UnrealEd",
				});
			
			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"LevelEditor",
				});
		}
	}
}
