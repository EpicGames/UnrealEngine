// Copyright Epic Games, Inc. All Rights Reserved.

#include "CalibrationPointComponentDetails.h"

#include "CalibrationPointComponent.h"
#include "CameraCalibrationCoreEditorModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "ProceduralMeshComponentDetails"

TSharedRef<IDetailCustomization> FCalibrationPointComponentDetails::MakeInstance()
{
	return MakeShareable(new FCalibrationPointComponentDetails);
}

void FCalibrationPointComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Calibration");

	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList = DetailBuilder.GetSelectedObjects();

	FCameraCalibrationCoreEditorModule& CameraCalibrationCoreEditorModule
		= FModuleManager::Get().LoadModuleChecked<FCameraCalibrationCoreEditorModule>("CameraCalibrationCoreEditor");
	
	for (TWeakPtr<ICalibrationPointComponentDetailsRow>& RowGenerator : CameraCalibrationCoreEditorModule.GetRegisteredCalibrationPointComponentDetailsRows())
	{
		if (!RowGenerator.IsValid())
		{
			continue;
		}

		FDetailWidgetRow& WidgetRow = Category.AddCustomRow(RowGenerator.Pin()->GetSearchString(), RowGenerator.Pin()->IsAdvanced());

		RowGenerator.Pin()->CustomizeRow(WidgetRow, SelectedObjectsList);
	}
}


#undef LOCTEXT_NAMESPACE
