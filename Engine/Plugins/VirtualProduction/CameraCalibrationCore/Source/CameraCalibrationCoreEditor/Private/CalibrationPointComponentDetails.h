// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class UCalibrationPointComponent;

class FCalibrationPointComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */
};
