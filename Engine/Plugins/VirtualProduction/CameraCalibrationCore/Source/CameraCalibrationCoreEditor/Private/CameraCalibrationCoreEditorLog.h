// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCameraCalibrationCoreEditor, Log, All);
