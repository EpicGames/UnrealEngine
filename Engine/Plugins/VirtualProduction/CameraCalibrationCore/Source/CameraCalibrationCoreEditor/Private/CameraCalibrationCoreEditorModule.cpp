// Copyright Epic Games, Inc. All Rights Reserved.


#include "CameraCalibrationCoreEditorModule.h"

#include "CoreMinimal.h"

#include "ActorFactories/ActorFactory.h"
#include "CalibrationPointComponent.h"
#include "CalibrationPointComponentDetails.h"
#include "CameraCalibrationCheckerboard.h"
#include "CameraCalibrationCoreEditorStyle.h"
#include "Editor.h"
#include "IPlacementModeModule.h"
#include "LevelEditorOutlinerSettings.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

LLM_DEFINE_TAG(CameraCalibrationCore_CameraCalibrationCoreEditor);
#define LOCTEXT_NAMESPACE "CameraCalibrationCoreEditor"

DEFINE_LOG_CATEGORY(LogCameraCalibrationCoreEditor);


void FCameraCalibrationCoreEditorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(CameraCalibrationCore_CameraCalibrationCoreEditor);

	FCameraCalibrationCoreEditorStyle::Get();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout(
		UCalibrationPointComponent::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FCalibrationPointComponentDetails::MakeInstance)
	);

	RegisterPlacementModeItems();
}

void FCameraCalibrationCoreEditorModule::ShutdownModule()
{
	LLM_SCOPE_BYTAG(CameraCalibrationCore_CameraCalibrationCoreEditor);

	if (!IsEngineExitRequested() && GEditor && UObjectInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyModule.UnregisterCustomClassLayout(UCalibrationPointComponent::StaticClass()->GetFName());

		UnregisterPlacementModeItems();
	}
}

void FCameraCalibrationCoreEditorModule::UnregisterPlacementModeItems()
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

	for (TOptional<FPlacementModeID>& PlaceActor : PlaceActors)
	{
		if (PlaceActor.IsSet())
		{
			PlacementModeModule.UnregisterPlaceableItem(*PlaceActor);
		}
	}

	PlaceActors.Empty();
}

const FPlacementCategoryInfo* FCameraCalibrationCoreEditorModule::GetVirtualProductionCategoryRegisteredInfo() const
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

	if (const FPlacementCategoryInfo* RegisteredInfo = PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
	{
		return RegisteredInfo;
	}
	else
	{
		FPlacementCategoryInfo Info(
			LOCTEXT("VirtualProductionCategoryName", "Virtual Production"),
			FSlateIcon(FCameraCalibrationCoreEditorStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.VirtualProduction"),
			FLevelEditorOutlinerBuiltInCategories::VirtualProduction(),
			TEXT("PMVirtualProduction"),
			25 // Determines where the category shows up in the list with respect to the others.
		);
		Info.ShortDisplayName = LOCTEXT("VirtualProductionShortCategoryName", "VP");
		IPlacementModeModule::Get().RegisterPlacementCategory(Info);

		return PlacementModeModule.GetRegisteredPlacementCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction());
	}
}

void FCameraCalibrationCoreEditorModule::RegisterPlacementModeItems()
{
	auto RegisterPlaceActors = [&]() -> void
	{
		LLM_SCOPE_BYTAG(CameraCalibrationCore_CameraCalibrationCoreEditor)
		if (!GEditor)
		{
			return;
		}

		const FPlacementCategoryInfo* Info = GetVirtualProductionCategoryRegisteredInfo();

		if (!Info)
		{
			UE_LOGF(LogCameraCalibrationCoreEditor, Warning, "Could not find or create VirtualProduction Place Actor Category");
			return;
		}

		// Register the Checkerboard
		PlaceActors.Add(IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
			*ACameraCalibrationCheckerboard::StaticClass(),
			FAssetData(ACameraCalibrationCheckerboard::StaticClass()),
			NAME_None,
			NAME_None,
			TOptional<FLinearColor>(),
			TOptional<int32>(),
			NSLOCTEXT("PlacementMode", "Checkerboard", "Checkerboard")
		)));
	};

	if (FApp::CanEverRender())
	{
		if (GEngine && GEngine->IsInitialized())
		{
			RegisterPlaceActors();
		}
		else
		{
			PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda(RegisterPlaceActors);
		}
	}
}

void FCameraCalibrationCoreEditorModule::RegisterCalibrationPointDetailsRow(const TWeakPtr<ICalibrationPointComponentDetailsRow> Row)
{
	// Take the opportunity to clean up the list
	RegisteredCalibrationPointComponentDetailsRows.RemoveAll([](TWeakPtr<ICalibrationPointComponentDetailsRow> Elem) {
		return !Elem.IsValid();
	});

	if (!Row.IsValid() || RegisteredCalibrationPointComponentDetailsRows.Contains(Row))
	{
		return;
	}

	RegisteredCalibrationPointComponentDetailsRows.Add(Row);
}

void FCameraCalibrationCoreEditorModule::UnregisterCalibrationPointDetailsRow(const TWeakPtr<ICalibrationPointComponentDetailsRow> Row)
{
	// Take the opportunity to clean up the list
	RegisteredCalibrationPointComponentDetailsRows.RemoveAll([](TWeakPtr<ICalibrationPointComponentDetailsRow> Elem) {
		return !Elem.IsValid();
	});

	if (!Row.IsValid())
	{
		return;
	}

	RegisteredCalibrationPointComponentDetailsRows.Remove(Row);
}

TArray<TWeakPtr<ICalibrationPointComponentDetailsRow>> FCameraCalibrationCoreEditorModule::GetRegisteredCalibrationPointComponentDetailsRows()
{
	return RegisteredCalibrationPointComponentDetailsRows;
}

IMPLEMENT_MODULE(FCameraCalibrationCoreEditorModule, CameraCalibrationCoreEditor);


#undef LOCTEXT_NAMESPACE
