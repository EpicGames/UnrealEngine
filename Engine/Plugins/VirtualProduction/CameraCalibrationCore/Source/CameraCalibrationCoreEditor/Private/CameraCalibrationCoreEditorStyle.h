// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"


/**
 * Implements the visual style of the camera calibration core  UI.
 */
class FCameraCalibrationCoreEditorStyle final : public FSlateStyleSet
{
public:

	/** Default constructor. */
	FCameraCalibrationCoreEditorStyle() : FSlateStyleSet("CameraCalibrationCoreEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		// Set placement browser icons
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/"));
		Set("PlacementBrowser.Icons.VirtualProduction", new IMAGE_BRUSH_SVG("Starship/Common/VirtualProduction", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	/** Virtual destructor. */
	virtual ~FCameraCalibrationCoreEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FCameraCalibrationCoreEditorStyle& Get()
	{
		static FCameraCalibrationCoreEditorStyle Inst;
		return Inst;
	}
};
