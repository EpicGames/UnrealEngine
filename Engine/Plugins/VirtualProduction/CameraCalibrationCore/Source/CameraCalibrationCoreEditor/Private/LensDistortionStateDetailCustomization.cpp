// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensDistortionStateDetailCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "LensDistortionModelHandlerBase.h"

FLensDistortionStateDetailCustomization::FLensDistortionStateDetailCustomization(TSubclassOf<ULensModel> InLensModel)
	: LensModel(InLensModel)
{
}

void FLensDistortionStateDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	HeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		InPropertyHandle->CreatePropertyValueWidget()
	];
}

void FLensDistortionStateDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(Index).ToSharedRef();

		const FName PropertyName = ChildPropertyHandle->GetProperty()->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensDistortionState, DistortionInfo))
		{
			CustomizeDistortionInfo(ChildPropertyHandle, ChildBuilder, CustomizationUtils);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensDistortionState, FocalLengthInfo))
		{
			CustomizeFocalLengthInfo(ChildPropertyHandle, ChildBuilder, CustomizationUtils);
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensDistortionState, ImageCenter))
		{
			CustomizeImageCenterInfo(ChildPropertyHandle, ChildBuilder, CustomizationUtils);
		}
		else
		{
			ChildBuilder.AddProperty(ChildPropertyHandle).ShowPropertyButtons(false);
		}
	}
}

void FLensDistortionStateDetailCustomization::CustomizeDistortionInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FDistortionInfo, Parameters))
		{
			TSharedPtr<IPropertyHandleArray> ParameterArrayHandle = ChildHandle->AsArray();
			uint32 NumParameters = 0;
			ParameterArrayHandle->GetNumElements(NumParameters);

			if (LensModel)
			{
				const TArray<FText> ParameterNames = LensModel.GetDefaultObject()->GetParameterDisplayNames();

				if (ParameterNames.Num() == NumParameters)
				{
					for (uint32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
					{
						TSharedRef<IPropertyHandle> ParameterHandle = ParameterArrayHandle->GetElement(ParameterIndex);
						ParameterHandle->SetPropertyDisplayName(ParameterNames[ParameterIndex]);
					}
				}
			}
				
			ChildBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
		}
	}
}

void FLensDistortionStateDetailCustomization::CustomizeFocalLengthInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Effectively, this does a ShowInnerPropertiesOnly, which does not work great by default with nested structs like this
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		ChildBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
	}
}

void FLensDistortionStateDetailCustomization::CustomizeImageCenterInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Effectively, this does a ShowInnerPropertiesOnly, which does not work great by default with nested structs like this
	uint32 NumChildren;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		ChildBuilder.AddProperty(ChildHandle).ShowPropertyButtons(false);
	}
}
