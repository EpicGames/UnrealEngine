// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "ICalibrationPointComponentDetailsRow.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"

class FPlacementModeID;
class IAssetTypeActions;

struct FPlacementCategoryInfo;

DECLARE_LOG_CATEGORY_EXTERN(LogCameraCalibrationCoreEditor, Log, All);


/**
 * Implements the CameraCalibrationCoreEditor module.
 */
class FCameraCalibrationCoreEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** Register a details row generator for CalibrationPointComponent */
	CAMERACALIBRATIONCOREEDITOR_API void RegisterCalibrationPointDetailsRow(const TWeakPtr<ICalibrationPointComponentDetailsRow> Row);

	/** Unregister a details row generator for CalibrationPointComponent */
	CAMERACALIBRATIONCOREEDITOR_API void UnregisterCalibrationPointDetailsRow(const TWeakPtr<ICalibrationPointComponentDetailsRow> Row);

	/** Returns a copy of the set of registered calibration point details rows */
	TArray<TWeakPtr<ICalibrationPointComponentDetailsRow>> GetRegisteredCalibrationPointComponentDetailsRows();

private:

	/** Register items to show up in the Place Actors panel. */
	void RegisterPlacementModeItems();

	/** Unregister items in Place Actors panel */
	void UnregisterPlacementModeItems();

	/** Gathers the Info on the Virtual Production Place Actors Category */
	const FPlacementCategoryInfo* GetVirtualProductionCategoryRegisteredInfo() const;

private:

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;
	FDelegateHandle PostEngineInitHandle;

	TArray<TOptional<FPlacementModeID>> PlaceActors;

	/** Array of registered details rows for CalibrationPointComponents */
	TArray<TWeakPtr<ICalibrationPointComponentDetailsRow>> RegisteredCalibrationPointComponentDetailsRows;
};
