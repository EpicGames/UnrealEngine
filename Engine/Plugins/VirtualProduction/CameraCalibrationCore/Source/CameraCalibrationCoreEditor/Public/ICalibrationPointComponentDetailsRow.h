// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDetailWidgetRow;
class FText;

class ICalibrationPointComponentDetailsRow
{
public:

	virtual ~ICalibrationPointComponentDetailsRow() = default;

public:

	/** Returns the search string for this details row */
	virtual FText GetSearchString() const = 0;

	/** True if this details row should appear in advanced section */
	virtual bool IsAdvanced() const = 0;

	/** 
	 * Customizes the given details widget row 
	 * 
	 * @param WidgetRow the detail widget row to customize
	 * @param SelectedObjectsList the list of objects selected that this row may apply to.
	 */
	virtual void CustomizeRow(
		FDetailWidgetRow& WidgetRow, 
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList) = 0;
};

