// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

#include "Models/LensModel.h"
#include "Templates/SubclassOf.h"

#define UE_API CAMERACALIBRATIONCOREEDITOR_API

class FLensDistortionStateDetailCustomization : public IPropertyTypeCustomization
{
public:
	UE_API FLensDistortionStateDetailCustomization(TSubclassOf<ULensModel> InLensModel);

	// Begin IDetailCustomization interface
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IDetailCustomization interface

private:
	/** Customize the FDistortionInfo struct property */
	UE_API void CustomizeDistortionInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** Customize the FFocalLengthInfo struct property */
	UE_API void CustomizeFocalLengthInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** Customize the FImageCenterInfo struct property */
	UE_API void CustomizeImageCenterInfo(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/** The LensModel used to customize the names of the properties in the distortion parameter array */
	TSubclassOf<ULensModel> LensModel;
};

#undef UE_API
